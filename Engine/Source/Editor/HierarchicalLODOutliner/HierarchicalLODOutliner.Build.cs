// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class HierarchicalLODOutliner : ModuleRules
{
    public HierarchicalLODOutliner(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PublicDependencyModuleNames.AddRange
        (
            new string[] {
				"Core",
                "InputCore",
                "CoreUObject",
				"RHI",
				"RenderCore",
				"Slate",
                "Engine",
                "UnrealEd",
                "PropertyEditor",
                "HierarchicalLODUtilities",
			}
        );

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				}
		);



		if (Target.bBuildEditor == true)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
					"EditorFramework",
    				"SlateCore",
					"ToolMenus",
                }
            );

            CircularlyReferencedDependentModules.AddRange(
                new string[] {
                    "UnrealEd"
                }
            );
        }
	}
}
