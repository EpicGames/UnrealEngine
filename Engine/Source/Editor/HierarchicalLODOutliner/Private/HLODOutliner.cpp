// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLODOutliner.h"

#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "DetailsViewArgs.h"
#include "DrawDebugHelpers.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/HLODProxy.h"
#include "Engine/LODActor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "GameFramework/WorldSettings.h"
#include "HLOD/HLODSetup.h"
#include "HLODOutlinerDragDrop.h"
#include "HLODTreeWidgetItem.h"
#include "HierarchicalLOD.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "IDetailsView.h"
#include "IHierarchicalLODUtilities.h"
#include "ITreeItem.h"
#include "Internationalization/Internationalization.h"
#include "LODActorItem.h"
#include "LODLevelItem.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Logging/MessageLog.h"
#include "Math/Box.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "StaticMeshActorItem.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Textures/SlateIcon.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "TreeItemID.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

class AHierarchicalLODVolume;
class FDragDropEvent;
class ITableRow;
class SWidget;
class UToolMenu;
struct FKeyEvent;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "HLODOutliner"

namespace HLODOutliner
{
	const int32 LOD_AUTO = -1;
	const int32 LOD_DISABLED = -2;

	SHLODOutliner::SHLODOutliner()		
	{
		bNeedsRefresh = true;
		CurrentWorld = nullptr;
		CurrentWorldSettings = nullptr;
		ForcedLODLevel = LOD_AUTO;
		bArrangeHorizontally = false;

		FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		HierarchicalLODUtilities = Module.GetUtilities();
	}

	SHLODOutliner::~SHLODOutliner()
	{
		DeregisterDelegates();
		FEditorDelegates::BeginPIE.RemoveAll(this);
		FEditorDelegates::EndPIE.RemoveAll(this);

		DestroySelectionActors();
		CurrentWorld = nullptr;
		HLODTreeRoot.Empty();
		SelectedNodes.Empty();		
		AllNodes.Empty();
		SelectedLODActors.Empty();
		LODLevelBuildFlags.Empty();
		LODLevelActors.Empty();
		PendingActions.Empty();
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void SHLODOutliner::Construct(const FArguments& InArgs)
	{
		CreateSettingsView();

		/** Holds all widgets for the profiler window like menu bar, toolbar and tabs. */
		MainContentPanel = SNew(SVerticalBox);
		ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("SettingsEditor.CheckoutWarningBorder"))
					.BorderBackgroundColor(FColor(166,137,0))							
					[
						SNew(SHorizontalBox)
						.Visibility_Lambda([this]() -> EVisibility 
						{
							bool bVisible = !bNeedsRefresh && CurrentWorld.IsValid() && HierarchicalLODUtilities->IsWorldUsedForStreaming(CurrentWorld.Get());
							return bVisible ? EVisibility::Visible : EVisibility::Collapsed;
						})

						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(4.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("SettingsEditor.WarningIcon"))
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("HLODDisabledSublevel", "Changing the HLOD settings is disabled for sub-levels"))
						]
					]
				]

				// Overlay slot for the main HLOD window area
				+ SVerticalBox::Slot()
				[
					MainContentPanel.ToSharedRef()
				]
			];

		// Disable panel if system is not enabled
		MainContentPanel->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SHLODOutliner::OutlinerEnabled)));

		SettingsView->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([]() -> bool { return !GetDefault<UHierarchicalLODSettings>()->bForceSettingsInAllMaps; })));

		MainContentPanel->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(4.0f, 4.0f, 4.0f, 0.0f))
				[
					CreateMainButtonWidgets()
				]
			];

		TSharedRef<SHLODTree> TreeViewWidget = CreateTreeviewWidget();

		TSharedRef<SWidget> ClusterWidgets = 
			SNew(SBorder)
			.BorderImage(&FAppStyle::GetWidgetStyle<FTableRowStyle>("TableView.Row").EvenRowBackgroundBrush)
			.Padding(0)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					CreateClusterButtonWidgets()
				]
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SNew(SScrollBorder, TreeViewWidget)
					[
						TreeViewWidget
					]
				]
			];

		TSharedRef<SWidget> DetailsWidgets = 
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(1.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("SettingsEditor.CheckoutWarningBorder"))
					.BorderBackgroundColor(FColor(166,137,0))							
					[
						SNew(SHorizontalBox)
						.Visibility_Lambda([this]() -> EVisibility 
						{
							return GetDefault<UHierarchicalLODSettings>()->bForceSettingsInAllMaps ? EVisibility::Visible : EVisibility::Collapsed;
						})

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(4.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Warning"))
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.0f)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(LOCTEXT("HLODForcedGlobally", "Project level HLOD Settings forced, changing the HLOD settings is disabled"))
							.ColorAndOpacity(FSlateColor(EStyleColor::Black))
						]
					]
				]
				+SVerticalBox::Slot()
				.Padding(2.0f, 1.0f)
				[
					SettingsView.ToSharedRef()
				]
			];

		MainContentPanel->AddSlot()
			.FillHeight(1.0f)
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(this, &SHLODOutliner::GetSpitterWidgetIndex)
				+SWidgetSwitcher::Slot()
				[
					SNew(SSplitter)
					.Style(FAppStyle::Get(), "DetailsView.Splitter")
					.Orientation(Orient_Horizontal)
					+ SSplitter::Slot()
					.Value(0.5)
					[
						ClusterWidgets
					]
					+SSplitter::Slot()
					.Value(0.5)
					[
						DetailsWidgets
					]
				]
				+SWidgetSwitcher::Slot()
				[
					SNew(SSplitter)
					.Style(FAppStyle::Get(), "DetailsView.Splitter")
					.Orientation(Orient_Vertical)
					+ SSplitter::Slot()
					.Value(0.5)
					[
						ClusterWidgets
					]
					+SSplitter::Slot()
					.Value(0.5)
					[
						DetailsWidgets
					]
				]
			];
		
		MainContentPanel->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 4.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("SettingsEditor.CheckoutWarningBorder"))
				.BorderBackgroundColor(FColor(166,137,0))
				[
					SNew(SHorizontalBox)
					.Visibility_Lambda([this]() -> EVisibility 
					{
						return bCachedNeedsBuild ? EVisibility::Visible : EVisibility::Collapsed;
					})

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(4.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Warning"))
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					.Padding(4.0f, 0.0f, 4.0f, 0.0f)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(LOCTEXT("HLODNeedsBuild", "Actors represented in HLOD have changed, generate proxy meshes to update."))
						.ColorAndOpacity(FSlateColor(EStyleColor::Black))
					]
				]
			];

		MainContentPanel->AddSlot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(8.0f, 8.0f, 0.0f, 8.0f))
				[
					SNew(SCheckBox)
					.Type(ESlateCheckBoxType::CheckBox)
					.IsChecked_Lambda([this]() { return (CurrentWorldSettings && CurrentWorldSettings->bGenerateSingleClusterForLevel) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState) {  if (CurrentWorldSettings) { CurrentWorldSettings->bGenerateSingleClusterForLevel = (NewState == ECheckBoxState::Checked); } })
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("GenerateSingleClusterLabel", "Generate Single Cluster for Level"))
					]
				]
			];

		if (!CurrentWorld.IsValid() || !CurrentWorld->IsPlayInEditor())
		{
			RegisterDelegates();
		}

		FEditorDelegates::BeginPIE.AddRaw(this, &SHLODOutliner::OnBeginPieEvent);
		FEditorDelegates::EndPIE.AddRaw(this, &SHLODOutliner::OnEndPieEvent);
	}

	TSharedRef<SWidget> SHLODOutliner::MakeToolBar()
	{
		FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this](){ GenerateClustersFromUI(); })),
			NAME_None,
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SHLODOutliner::GetRegenerateClustersText)),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SHLODOutliner::GetRegenerateClustersTooltip)),
			TAttribute<FSlateIcon>()
		);

		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this](){ GenerateProxyMeshesFromUI(); }),
				FCanExecuteAction::CreateSP(this, &SHLODOutliner::CanBuildLODActors)),
			NAME_None,
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SHLODOutliner::GetBuildText)),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SHLODOutliner::GetBuildLODActorsTooltipText)),
			TAttribute<FSlateIcon>()
		);

		ToolBarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateLambda([this] { BuildClustersAndMeshesFromUI(); })),
			NAME_None,
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SHLODOutliner::GetForceBuildText)),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SHLODOutliner::GetForceBuildToolTip)),
			TAttribute<FSlateIcon>()
		);

		ToolBarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateLambda([this]() { HandleSaveAll(); })),
			NAME_None,
			FText(),
			LOCTEXT("SaveAllToolTip", "Saves all external HLOD data: Meshes, materials etc."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "AssetEditor.SaveAsset")
		);

		return ToolBarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> SHLODOutliner::CreateMainButtonWidgets()
	{
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(2.0f, 4.0f))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(FMargin(2.0f))
				.AutoWidth()
				[
					MakeToolBar()
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(FMargin(2.0f))
				[
					CreateForcedViewWidget()
				]
			];
	}

	TSharedRef<SWidget> SHLODOutliner::CreateClusterButtonWidgets()
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 2.0f))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(FMargin(6.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "DialogButtonText")
						.Text(LOCTEXT("ClustersLabel", "Clusters"))
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(2.0f))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(0.0f)
						.HAlign(HAlign_Center)
						.OnClicked(this, &SHLODOutliner::GenerateClustersFromUI)
						.ToolTipText(LOCTEXT("GenerateClusterToolTip", "Generates clusters (but not proxy meshes) for meshes in the level"))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						]
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(2.0f))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ContentPadding(0.0f)
						.HAlign(HAlign_Center)
						.OnClicked_Lambda([this]() { return HandleDeleteHLODs(); })
						.IsEnabled(this, &SHLODOutliner::CanDeleteHLODs)
						.ToolTipText(LOCTEXT("DeleteClusterToolTip", "Deletes all clusters in the level"))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						]
					]
				]
			];
	}

	TSharedRef<SHLODOutliner::SHLODTree> SHLODOutliner::CreateTreeviewWidget()
	{
		return SAssignNew(TreeView, SHLODTree)
			.TreeItemsSource(&HLODTreeRoot)
			.OnGenerateRow(this, &SHLODOutliner::OnOutlinerGenerateRow)
			.OnGetChildren(this, &SHLODOutliner::OnOutlinerGetChildren)
			.OnSelectionChanged(this, &SHLODOutliner::OnOutlinerSelectionChanged)
			.OnMouseButtonDoubleClick(this, &SHLODOutliner::OnOutlinerDoubleClick)
			.OnContextMenuOpening(this, &SHLODOutliner::OnOpenContextMenu)
			.OnExpansionChanged(this, &SHLODOutliner::OnItemExpansionChanged)			
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column("SceneActorName")
				.DefaultLabel(LOCTEXT("SceneActorName", "Actor Name"))
				.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.FillWidth(0.3f)				
				+ SHeaderRow::Column("RawTriangleCount")
				.DefaultLabel(LOCTEXT("RawTriangleCount", "Tri Original"))
				.DefaultTooltip(LOCTEXT("RawTriangleCountToolTip", "Original Number of Triangles in a LOD Mesh"))
				.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.FillWidth(0.2f)
				+ SHeaderRow::Column("ReducedTriangleCount")
				.DefaultLabel(LOCTEXT("ReducedTriangleCount", "Tri Reduced"))
				.DefaultTooltip(LOCTEXT("ReducedTriangleCountToolTip", "Reduced Number of Triangles in a LOD Mesh"))
				.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.FillWidth(0.2f)
				+ SHeaderRow::Column("ReductionPercentage")
				.DefaultLabel(LOCTEXT("ReductionPercentage", "% Retained"))
				.DefaultTooltip(LOCTEXT("ReductionPercentageToolTip", "Percentage of Triangle Reduction in a LOD Mesh"))
				.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.FillWidth(0.2f)
				+ SHeaderRow::Column("Level")
				.DefaultLabel(LOCTEXT("Level", "Level"))
				.DefaultTooltip(LOCTEXT("LevelToolTip", "Persistent Level of a LOD Mesh"))
				.HeaderContentPadding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				.FillWidth(0.1f)
			);
	}

	TSharedRef<SWidget> SHLODOutliner::CreateForcedViewWidget()
	{
		return SNew(SComboButton)
				.ContentPadding(FMargin(4.0f, 2.0f))
				.ForegroundColor(FLinearColor::White)
				.ButtonStyle(FAppStyle::Get(), "FlatButton")
				.HasDownArrow(true)
				.OnGetMenuContent(this, &SHLODOutliner::GetForceLevelMenuContent)
				.ToolTipText(LOCTEXT("ForcedLODButtonTooltip", "Choose the LOD level to view."))
				.ButtonContent()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "DialogButtonText")
					.Text(this, &SHLODOutliner::HandleForceLevelText)
				];
	}

	void SHLODOutliner::CreateSettingsView()
	{
		// Create a property view
		FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.NotifyHook = this;
		DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
		DetailsViewArgs.bShowOptions = false;

		SettingsView = EditModule.CreateDetailView(DetailsViewArgs);

		struct Local
		{
			/** Delegate to show all properties */
			static bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent, bool bInShouldShowNonEditable)
			{
				const char* CategoryNames[5] =
				{
					"HLODSystem",
					"ProxySettings",
					"LandscapeCulling",
					"MeshSettings",
					"MaterialSettings"
				};

				FString CategoryName = PropertyAndParent.Property.GetMetaData("Category");
				
				// Exceptions
				// This one is shown at the bottom of the windows, outside of the properties
				if (CategoryName == "HLODSystem" && PropertyAndParent.Property.GetName() == "bGenerateSingleClusterForLevel")
				{
					return false;
				}

				// General case
				for (uint32 CategoryIndex = 0; CategoryIndex < 5; ++CategoryIndex)
				{
					if (CategoryName == CategoryNames[CategoryIndex])
					{
						return true;
					}
				}

				return false;
			}
		};

		SettingsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateStatic(&Local::IsPropertyVisible, true));
		SettingsView->SetDisableCustomDetailLayouts(true);
	}

	void SHLODOutliner::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		// Get a collection of items and folders which were formerly collapsed
		const FParentsExpansionState ExpansionStateInfo = GetParentsExpansionState();

		if (bNeedsRefresh)
		{
			Populate();
		}
		
		// Draw currently selected HLOD clusters in the treeview as spheres in the level
		for (AActor* Actor : SelectedLODActors)
		{
			const FBox BoundingBox = Actor->GetComponentsBoundingBox(true);		
			DrawCircle(CurrentWorld.Get(), BoundingBox.GetCenter(), FVector(1, 0, 0), FVector(0, 1, 0), FColor::Red, BoundingBox.GetExtent().Size(), 32);
			DrawCircle(CurrentWorld.Get(), BoundingBox.GetCenter(), FVector(1, 0, 0), FVector(0, 0, 1), FColor::Red, BoundingBox.GetExtent().Size(), 32);
			DrawCircle(CurrentWorld.Get(), BoundingBox.GetCenter(), FVector(0, 1, 0), FVector(0, 0, 1), FColor::Red, BoundingBox.GetExtent().Size(), 32);
		}

		bool bChangeMade = false;

		// Only deal with 256 at a time
		const int32 End = FMath::Min(PendingActions.Num(), 512);
		for (int32 Index = 0; Index < End; ++Index)
		{
			auto& PendingAction = PendingActions[Index];
			switch (PendingAction.Type)
			{
			case FOutlinerAction::AddItem:
				bChangeMade |= AddItemToTree(PendingAction.Item, PendingAction.ParentItem);
				break;

			case FOutlinerAction::MoveItem:
				MoveItemInTree(PendingAction.Item, PendingAction.ParentItem);
				bChangeMade = true;
				break;

			case FOutlinerAction::RemoveItem:
				RemoveItemFromTree(PendingAction.Item);
				bChangeMade = true;
				break;
			default:
				check(false);
				break;
			}
		}
		PendingActions.RemoveAt(0, End);
				
		if (bChangeMade)
		{
			// Restore the expansion states
			SetParentsExpansionState(ExpansionStateInfo);

			// Restore expansion states
			TreeView->RequestTreeRefresh();		
		}

		bArrangeHorizontally = AllottedGeometry.Size.X > AllottedGeometry.Size.Y;
	}

	void SHLODOutliner::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	}

	void SHLODOutliner::OnMouseLeave(const FPointerEvent& MouseEvent)
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);
	}

	FReply SHLODOutliner::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	FReply SHLODOutliner::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		return SCompoundWidget::OnDrop(MyGeometry, DragDropEvent);
	}

	FReply SHLODOutliner::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		return SCompoundWidget::OnDragOver(MyGeometry, DragDropEvent);
	}

	void SHLODOutliner::PostUndo(bool bSuccess)
	{
		FullRefresh();
	}

	int32 SHLODOutliner::GetSpitterWidgetIndex() const
	{
		// Split vertically or horizontally based on dimensions
		return bArrangeHorizontally ? 0 : 1;
	}

	bool SHLODOutliner::HasHLODActors() const
	{
		for(const TArray<TWeakObjectPtr<ALODActor>>& LODActorArray : LODLevelActors)
		{
			for(TWeakObjectPtr<ALODActor> LODActor : LODActorArray)
			{
				if(LODActor.IsValid())
				{
					return true;
				}
			}
		}

		return false;
	}

	EActiveTimerReturnType SHLODOutliner::UpdateNeedsBuildFlagTimer(double InCurrentTime, float InDeltaTime)
	{
		bCachedNeedsBuild = (CurrentWorld.IsValid() && CurrentWorld->HierarchicalLODBuilder && CurrentWorld->HierarchicalLODBuilder->NeedsBuild());

		return EActiveTimerReturnType::Continue;
	}

	FText SHLODOutliner::GetBuildText() const
	{
		return LOCTEXT("BuildMeshes", "Generate Proxy Meshes");
	}

	FText SHLODOutliner::GetForceBuildText() const
	{
		return LOCTEXT("RebuildAllClustersAndMeshes", "Build All");
	}

	FText SHLODOutliner::GetForceBuildToolTip() const
	{
		return LOCTEXT("BuildClustersAndMeshesToolTip", "Re-generates clusters and then proxy meshes for each of the generated clusters in the level. This dirties the level.");
	}

	FReply SHLODOutliner::HandleDeleteHLODs()
	{
		if (CurrentWorld.IsValid())
		{
			LODLevelActors.Empty();
			CurrentWorld->HierarchicalLODBuilder->ClearHLODs();
		}
		
		ResetLODLevelForcing();
		SelectedLODActors.Empty();

		FullRefresh();
		return FReply::Handled();
	}

	bool SHLODOutliner::CanDeleteHLODs() const
	{
		return HasHLODActors();
	}

	FReply SHLODOutliner::HandlePreviewHLODs()
	{
		if (CurrentWorld.IsValid())
		{
			CurrentWorld->HierarchicalLODBuilder->PreviewBuild();
		}

		FMessageLog("HLODResults").Open();

		FullRefresh();
		return FReply::Handled();
	}

	FReply SHLODOutliner::HandleBuildLODActors()
	{
		if (CurrentWorld.IsValid())
		{
			auto Build = [this](bool bForce = true)
			{
				CloseOpenAssetEditors();

				DestroySelectionActors();
			
				CurrentWorld->HierarchicalLODBuilder->BuildMeshesForLODActors(bForce);
				SetForcedLODLevel(ForcedLODLevel);	
			};

			// Check if we have any dirty and pop a toast saying no rebuild needed (with optional force build).
			if(!CurrentWorld->HierarchicalLODBuilder->NeedsBuild(true))
			{
				FNotificationInfo Info(LOCTEXT("NoLODActorsNeedBuilding", "No LOD actors need building."));
				Info.ButtonDetails.Add(
					FNotificationButtonInfo(
						LOCTEXT("ForceBuildButtonLabel", "Force Build"), 
						LOCTEXT("ForceBuildButtonTooltip", "Force a rebuild of all LOD actors."), 
						FSimpleDelegate::CreateLambda(Build), SNotificationItem::CS_None));
				Info.ExpireDuration = 6.0f;
				FSlateNotificationManager::Get().AddNotification(Info);
			}
			else
			{
				Build(false);
			}
		}

		FMessageLog("HLODResults").Open();		

		return FReply::Handled();
	}

	bool SHLODOutliner::CanBuildLODActors() const
	{
		return HasHLODActors();
	}

	FText SHLODOutliner::GetBuildLODActorsTooltipText() const
	{
		return bCachedNeedsBuild ? LOCTEXT("GenerateProxyMeshesToolTip", "Generates a proxy mesh for each cluster in the level. This only dirties the generated mesh.") : LOCTEXT("GenerateProxyMeshesNoBuildNeededToolTip", "Generates a proxy mesh for each cluster in the level. This only dirties the generated mesh.\nCurrently no actors are dirty, so no build is necessary.");
	}

	FReply SHLODOutliner::HandleForceBuildLODActors()
	{
		CloseOpenAssetEditors();

		if (CurrentWorld.IsValid())
		{
			DestroySelectionActors();
			LODLevelActors.Empty();
			CurrentWorld->HierarchicalLODBuilder->ClearHLODs();
			CurrentWorld->HierarchicalLODBuilder->PreviewBuild();
			CurrentWorld->HierarchicalLODBuilder->BuildMeshesForLODActors(true);
		}

		SelectedLODActors.Empty();
		ResetLODLevelForcing();
		FullRefresh();

		FMessageLog("HLODResults").Open();		

		return FReply::Handled();
	}

	FReply SHLODOutliner::HandleForceRefresh()
	{
		FullRefresh();

		return FReply::Handled();
	}

	FReply SHLODOutliner::HandleSaveAll()
	{
		if (CurrentWorld.IsValid())
		{		
			CurrentWorld->HierarchicalLODBuilder->SaveMeshesForActors();
		}

		return FReply::Handled();
	}
	
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	FReply SHLODOutliner::GenerateClustersFromUI()
	{
		return HandlePreviewHLODs();
	}

	FReply SHLODOutliner::GenerateProxyMeshesFromUI()
	{
		return HandleBuildLODActors();
	}

	FReply SHLODOutliner::BuildClustersAndMeshesFromUI()
	{
		return HandleForceBuildLODActors();
	}

	FText SHLODOutliner::GetGenerateClustersText() const
	{
		return LOCTEXT("GenerateClustersLabel", "Generate Clusters");
	}

	FText SHLODOutliner::GetGenerateClustersTooltip() const
	{
		return LOCTEXT("GenerateClusterToolTip", "Generates clusters (but not proxy meshes) for meshes in the level");
	}

	FText SHLODOutliner::GetRegenerateClustersText() const
	{
		return LOCTEXT("RegenerateClustersLabel", "Regenerate Clusters");
	}

	FText SHLODOutliner::GetRegenerateClustersTooltip() const
	{
		return LOCTEXT("RegenerateClusterToolTip", "Regenerates clusters (but not proxy meshes) for meshes in the level");
	}

	void SHLODOutliner::OnBeginPieEvent(bool bIsSimulating)
	{
		DeregisterDelegates();
		FullRefresh();
	}

	void SHLODOutliner::OnEndPieEvent(bool bIsSimulating)
	{
		RegisterDelegates();
		FullRefresh();
	}

	void SHLODOutliner::RegisterDelegates()
	{
		FEditorDelegates::MapChange.AddSP(this, &SHLODOutliner::OnMapChange);
		FEditorDelegates::NewCurrentLevel.AddSP(this, &SHLODOutliner::OnNewCurrentLevel);
		FEditorDelegates::OnMapOpened.AddSP(this, &SHLODOutliner::OnMapLoaded);

		FWorldDelegates::LevelAddedToWorld.AddSP(this, &SHLODOutliner::OnLevelAdded);
		FWorldDelegates::LevelRemovedFromWorld.AddSP(this, &SHLODOutliner::OnLevelRemoved);

		GEngine->OnLevelActorListChanged().AddSP(this, &SHLODOutliner::FullRefresh);
		GEngine->OnLevelActorAdded().AddSP(this, &SHLODOutliner::OnLevelActorsAdded);
		GEngine->OnLevelActorDeleted().AddSP(this, &SHLODOutliner::OnLevelActorsRemoved);
		GEngine->OnActorMoved().AddSP(this, &SHLODOutliner::OnActorMovedEvent);

		// Selection change
		USelection::SelectionChangedEvent.AddRaw(this, &SHLODOutliner::OnLevelSelectionChanged);
		USelection::SelectObjectEvent.AddRaw(this, &SHLODOutliner::OnLevelSelectionChanged);

		// HLOD related events
		GEditor->OnHLODActorMoved().AddSP(this, &SHLODOutliner::OnHLODActorMovedEvent);
		GEditor->OnHLODActorAdded().AddSP(this, &SHLODOutliner::OnHLODActorAddedEvent);
		GEditor->OnHLODTransitionScreenSizeChanged().AddSP(this, &SHLODOutliner::OnHLODTransitionScreenSizeChangedEvent);
		GEditor->OnHLODLevelsArrayChanged().AddSP(this, &SHLODOutliner::OnHLODLevelsArrayChangedEvent);
		GEditor->OnHLODActorRemovedFromCluster().AddSP(this, &SHLODOutliner::OnHLODActorRemovedFromClusterEvent);

		// Register to update when an undo/redo operation has been called to update our list of actors
		GEditor->RegisterForUndo(this);

		ActiveTimerHandle = RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SHLODOutliner::UpdateNeedsBuildFlagTimer));
	}

	void SHLODOutliner::DeregisterDelegates()
	{
		FEditorDelegates::MapChange.RemoveAll(this);
		FEditorDelegates::NewCurrentLevel.RemoveAll(this);
		FEditorDelegates::OnMapOpened.RemoveAll(this);

		FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
		FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

		USelection::SelectionChangedEvent.RemoveAll(this);
		USelection::SelectObjectEvent.RemoveAll(this);

		if (GEngine)
		{
			GEngine->OnLevelActorListChanged().RemoveAll(this);
			GEngine->OnLevelActorAdded().RemoveAll(this);
			GEngine->OnLevelActorDeleted().RemoveAll(this);
			GEngine->OnActorMoved().RemoveAll(this);
		}	
		
		if (GEditor && UObjectInitialized())
		{
			GEditor->OnHLODActorMoved().RemoveAll(this);
			GEditor->OnHLODActorAdded().RemoveAll(this);
			GEditor->OnHLODTransitionScreenSizeChanged().RemoveAll(this);
			GEditor->OnHLODLevelsArrayChanged().RemoveAll(this);
			GEditor->OnHLODActorRemovedFromCluster().RemoveAll(this);

			// Deregister for Undo callbacks
			GEditor->UnregisterForUndo(this);
		}

		UnRegisterActiveTimer(ActiveTimerHandle.ToSharedRef());
	}

	void SHLODOutliner::ForceViewLODActor()
	{
		if (CurrentWorld.IsValid())
		{
			const FScopedTransaction Transaction(LOCTEXT("UndoAction_LODLevelForcedView", "LOD Level Forced View"));

			// This call came from a context menu
			auto SelectedItems = TreeView->GetSelectedItems();

			// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
			for (auto SelectedItem : SelectedItems)
			{
				FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());

				if (ActorItem->LODActor.IsValid())
				{
					ActorItem->LODActor->Modify();
					ActorItem->LODActor->ToggleForceView();
				}
			}
		}
	}

	bool SHLODOutliner::AreHLODsBuild() const
	{
		bool bHLODsBuild = true;
		for (bool Build : LODLevelBuildFlags)
		{
			bHLODsBuild &= Build;
		}

		return (LODLevelTransitionScreenSizes.Num() > 0 && bHLODsBuild);
	}

	FText SHLODOutliner::HandleForceLevelText() const
	{
		return ForcedLODLevel == LOD_AUTO ? LOCTEXT("AutoLOD", "LOD Auto") : 
			   ForcedLODLevel == LOD_DISABLED ? LOCTEXT("DisabledLOD", "LOD Disabled") : 
			   FText::Format(LOCTEXT("LODLevelFormat", "LOD {0}"), FText::AsNumber(ForcedLODLevel));
	}

	TSharedRef<SWidget> SHLODOutliner::GetForceLevelMenuContent() const
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		// Auto LOD
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoLOD", "LOD Auto"),
			LOCTEXT("AutoLODTooltip", "Determine LOD level automatically"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<SHLODOutliner*>(this), &SHLODOutliner::SetForcedLODLevel, LOD_AUTO), 
				FCanExecuteAction(), 
				FGetActionCheckState::CreateLambda([this](){ return ForcedLODLevel == LOD_AUTO ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton);

		// Auto LOD
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisabledLOD", "LOD Disabled"),
			LOCTEXT("DisabledLODTooltip", "Disable LOD and show original scene"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(const_cast<SHLODOutliner*>(this), &SHLODOutliner::SetForcedLODLevel, LOD_DISABLED),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([this]() { return ForcedLODLevel == LOD_DISABLED ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
			NAME_None,
			EUserInterfaceActionType::RadioButton);

		if(LODLevelTransitionScreenSizes.Num() > 0)
		{
			MenuBuilder.BeginSection("ForcedLODLevels", LOCTEXT("ForcedLODLevel", "Forced LOD Level"));
			{
				// Entry for each LOD level
				for(int32 LODIndex = 0; LODIndex < LODLevelTransitionScreenSizes.Num(); ++LODIndex)
				{
					MenuBuilder.AddMenuEntry(
						FText::Format(LOCTEXT("LODLevelFormat", "LOD {0}"), FText::AsNumber(LODIndex)),
						FText::Format(LOCTEXT("LODLevelTooltipFormat", "Force LOD to level {0}"), FText::AsNumber(LODIndex)),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(const_cast<SHLODOutliner*>(this), &SHLODOutliner::SetForcedLODLevel, LODIndex), 
							FCanExecuteAction(), 
							FGetActionCheckState::CreateLambda([this, LODIndex](){ return ForcedLODLevel == LODIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })),
						NAME_None,
						EUserInterfaceActionType::RadioButton);
				}
			}
			MenuBuilder.EndSection();
		}

		return MenuBuilder.MakeWidget();
	}

	void SHLODOutliner::RestoreForcedLODLevel(int32 LODLevel)
	{
		if (LODLevel == LOD_AUTO)
		{
			return;
		}

		if (CurrentWorld.IsValid())
		{
			for (auto LevelActors : LODLevelActors)
			{
				for (auto LODActor : LevelActors)
				{
					if (LODActor->LODLevel == LODLevel + 1)
					{
						LODActor->SetForcedView(false);
					}
					else
					{
						LODActor->SetHiddenFromEditorView(false, LODLevel + 1);
					}
				}
			}
		}
	}

	void SHLODOutliner::SetForcedLODLevel(int32 LODLevel)
	{
		RestoreForcedLODLevel(ForcedLODLevel);

		if (LODLevel == LOD_AUTO)
		{
			ForcedLODLevel = LODLevel;
			return;
		}

		if (CurrentWorld.IsValid())
		{
			auto Level = CurrentWorld->GetCurrentLevel();
			for (auto LevelActors : LODLevelActors)
			{
				for (auto LODActor : LevelActors )
				{
					if (LODActor->LODLevel == LODLevel + 1)
					{
						LODActor->SetForcedView(true);
					}
					else
					{
						LODActor->SetHiddenFromEditorView(true, LODLevel + 1);
					}
				}
			}
		}
		ForcedLODLevel = LODLevel;
	}

	void SHLODOutliner::ResetLODLevelForcing()
	{
		RestoreForcedLODLevel(ForcedLODLevel);
		SetForcedLODLevel(LOD_AUTO);
	}

	void SHLODOutliner::CreateHierarchicalVolumeForActor()
	{		
		// This call came from a context menu
		auto SelectedItems = TreeView->GetSelectedItems();

		// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
		for (auto SelectedItem : SelectedItems)
		{
			FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());
			ALODActor* LODActor = ActorItem->LODActor.Get();

			AHierarchicalLODVolume* Volume = HierarchicalLODUtilities->CreateVolumeForLODActor(LODActor, nullptr);
			check(Volume);
		}		
	}

	void SHLODOutliner::BuildLODActor()
	{
		if (CurrentWorld.IsValid())
		{
			// This call came from a context menu
			auto SelectedItems = TreeView->GetSelectedItems();
			
			FScopedSlowTask SlowTask((float)SelectedItems.Num(), FText::Format(LOCTEXT("SHLODOutliner_BuildProxy", "Building Proxy {0}|plural(one=Mesh,other=Meshes)"), SelectedItems.Num()));
			SlowTask.MakeDialog();

			// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
			for (auto SelectedItem : SelectedItems )
			{
				FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());
				if (ActorItem->LODActor->HasValidSubActors())
				{
					auto Parent = ActorItem->GetParent();

					ITreeItem::TreeItemType Type = Parent->GetTreeItemType();
					if (Type == ITreeItem::HierarchicalLODLevel)
					{
						FLODLevelItem* LevelItem = (FLODLevelItem*)(Parent.Get());
						if (!ActorItem->LODActor->IsBuilt(true))
						{
							CurrentWorld->HierarchicalLODBuilder->BuildMeshForLODActor(ActorItem->LODActor.Get(), LevelItem->LODLevelIndex);
						}
					}
				}

				SlowTask.EnterProgressFrame();
			}
			
			SetForcedLODLevel(ForcedLODLevel);
			TreeView->RequestScrollIntoView(SelectedItems[0]);
		}
		
		// Show message log if there was an HLOD message
		FMessageLog("HLODResults").Open();
	}

	void SHLODOutliner::RebuildLODActor()
	{
		if (CurrentWorld.IsValid())
		{
			CloseOpenAssetEditors();

			// This call came from a context menu
			auto SelectedItems = TreeView->GetSelectedItems();

			FScopedSlowTask SlowTask((float)SelectedItems.Num(), FText::Format(LOCTEXT("SHLODOutliner_RebuildProxy", "Rebuild Proxy {0}|plural(one=Mesh,other=Meshes)"), SelectedItems.Num()));
			SlowTask.MakeDialog();

			// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
			for (auto SelectedItem : SelectedItems)
			{
				FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());
				if (ActorItem->LODActor->HasValidSubActors())
				{
					auto Parent = ActorItem->GetParent();

					ITreeItem::TreeItemType Type = Parent->GetTreeItemType();
					if (Type == ITreeItem::HierarchicalLODLevel)
					{
						FLODLevelItem* LevelItem = (FLODLevelItem*)(Parent.Get());
						CurrentWorld->HierarchicalLODBuilder->BuildMeshForLODActor(ActorItem->LODActor.Get(), LevelItem->LODLevelIndex);
					}
				}

				SlowTask.EnterProgressFrame();
			}

			SetForcedLODLevel(ForcedLODLevel);
			TreeView->RequestScrollIntoView(SelectedItems[0]);
		}
			
		// Show message log if there was an HLOD message
		FMessageLog("HLODResults").Open();
	}

	void SHLODOutliner::SelectLODActor()
	{
		if (CurrentWorld.IsValid())
		{
			// This call came from a context menu
			auto SelectedItems = TreeView->GetSelectedItems();

			// Empty selection and setup for multi-selection
			EmptySelection();
			StartSelection();


			bool bChanged = false;
			// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
			for (auto SelectedItem : SelectedItems)
			{
				FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());

				if (ActorItem->LODActor.IsValid())
				{				
					SelectActorInViewport(ActorItem->LODActor.Get(), 0);			
					bChanged = true;
				}
			}
			
			// Done selecting actors
			EndSelection(bChanged);
		}
	}

	void SHLODOutliner::DeleteCluster()
	{
		// This call came from a context menu
		auto SelectedItems = TreeView->GetSelectedItems();
		// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
		for (auto SelectedItem : SelectedItems)
		{
			FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());
			ALODActor* LODActor = ActorItem->LODActor.Get();			
						
			SelectedLODActors.RemoveAll([LODActor](const AActor* Actor) { return Actor == LODActor; });

			HierarchicalLODUtilities->DestroyLODActor(LODActor);
		}

		ResetLODLevelForcing();
		FullRefresh();
	}

	void SHLODOutliner::RemoveStaticMeshActorFromCluster()
	{
		if (CurrentWorld.IsValid())
		{
			const FScopedTransaction Transaction(LOCTEXT("UndoAction_RemoveStaticMeshActorFromCluster", "Removed Static Mesh Actor From Cluster"));

			// This call came from a context menu
			auto SelectedItems = TreeView->GetSelectedItems();

			// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
			for (auto SelectedItem : SelectedItems)
			{
				FStaticMeshActorItem* ActorItem = (FStaticMeshActorItem*)(SelectedItem.Get());
				auto Parent = ActorItem->GetParent();

				ITreeItem::TreeItemType Type = Parent->GetTreeItemType();
				if (Type == ITreeItem::HierarchicalLODActor)
				{
					AActor* Actor = ActorItem->StaticMeshActor.Get();
					
					if (HierarchicalLODUtilities->RemoveActorFromCluster(Actor))
					{
						PendingActions.Emplace(FOutlinerAction::RemoveItem, SelectedItem);
					}
				}
			}
		}
	}

	void SHLODOutliner::ExcludeFromClusterGeneration()
	{
		// This call came from a context menu
		auto SelectedItems = TreeView->GetSelectedItems();

		// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
		for (auto SelectedItem : SelectedItems)
		{
			FStaticMeshActorItem* ActorItem = (FStaticMeshActorItem*)(SelectedItem.Get());
			HierarchicalLODUtilities->ExcludeActorFromClusterGeneration(ActorItem->StaticMeshActor.Get());
		}		
	}

	void SHLODOutliner::RemoveLODActorFromCluster()
	{
		if (CurrentWorld.IsValid())
		{
			// This call came from a context menu
			auto SelectedItems = TreeView->GetSelectedItems();

			// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
			for (auto SelectedItem : SelectedItems)
			{
				FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());
				auto Parent = ActorItem->GetParent();

				ITreeItem::TreeItemType Type = Parent->GetTreeItemType();
				if (Type == ITreeItem::HierarchicalLODActor)
				{
					AActor* Actor = ActorItem->LODActor.Get();
					checkf(Actor != nullptr, TEXT("Invalid actor in tree view"));
					
					if (HierarchicalLODUtilities->RemoveActorFromCluster(Actor))
					{
						PendingActions.Emplace(FOutlinerAction::RemoveItem, SelectedItem);
					}					
				}
			}
		}
	}

	void SHLODOutliner::CreateClusterFromActors(const TArray<AActor*>& Actors, uint32 LODLevelIndex)
	{
		HierarchicalLODUtilities->CreateNewClusterFromActors(CurrentWorld.Get(), CurrentWorldSettings, Actors, LODLevelIndex);
	}

	void SHLODOutliner::SelectContainedActors()
	{
		// This call came from a context menu
		auto SelectedItems = TreeView->GetSelectedItems();

		// Empty selection and setup for multi-selection
		EmptySelection();
		StartSelection();

		// Loop over all selected items (context menu can't be called with multiple items selected that aren't of the same types)
		for (auto SelectedItem : SelectedItems)
		{
			FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());

			ALODActor* LODActor = ActorItem->LODActor.Get();
			SelectContainedActorsInViewport(LODActor, 0);
		}

		// Done selecting actors
		EndSelection(true);
	}

	void SHLODOutliner::UpdateDrawDistancesForLODLevel(const uint32 LODLevelIndex)
	{
		if (CurrentWorld.IsValid())
		{
			// Loop over all (streaming-)levels in the world
			for (ULevel* Level : CurrentWorld->GetLevels())
			{
				// For each LOD actor in the world update the drawing/transition distance
				for (AActor* Actor : Level->Actors)
				{
					ALODActor* LODActor = Cast<ALODActor>(Actor);
					if (LODActor)
					{
						if (LODActor->LODLevel == LODLevelIndex + 1)
						{
							if (LODActor->IsBuilt(true) && LODActor->GetStaticMeshComponent())
							{
								const float ScreenSize = LODActor->bOverrideTransitionScreenSize ? LODActor->TransitionScreenSize : LODLevelTransitionScreenSizes[LODLevelIndex];
								LODActor->RecalculateDrawingDistance(ScreenSize);
							}
						}
					}
				}
			}
		}
	}

	void SHLODOutliner::RemoveLODLevelActors(const int32 HLODLevelIndex)
	{
		if (CurrentWorld.IsValid())
		{
			HierarchicalLODUtilities->DeleteLODActorsInHLODLevel(CurrentWorld.Get(), HLODLevelIndex);
		}
	}

	TSharedRef<ITableRow> SHLODOutliner::OnOutlinerGenerateRow(FTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedRef<ITableRow> Widget = SNew(SHLODWidgetItem, OwnerTable)
			.TreeItemToVisualize(InTreeItem)
			.Outliner(this)
			.World(CurrentWorld.Get());

		return Widget;
	}

	void SHLODOutliner::OnOutlinerGetChildren(FTreeItemPtr InParent, TArray<FTreeItemPtr>& OutChildren)
	{		
		for (auto& WeakChild : InParent->GetChildren())
		{
			auto Child = WeakChild.Pin();
			// Should never have bogus entries in this list
			check(Child.IsValid());
			OutChildren.Add(Child);
		}
	}

	void SHLODOutliner::OnOutlinerSelectionChanged(FTreeItemPtr TreeItem, ESelectInfo::Type SelectInfo)
	{
		if (SelectInfo == ESelectInfo::Direct)
		{
			return;
		}

		TArray<FTreeItemPtr> NewSelectedNodes = TreeView->GetSelectedItems();
		// Make sure that we do not actually change selection when the users selects a HLOD level node
		if (NewSelectedNodes.ContainsByPredicate([](FTreeItemPtr Item) -> bool { return Item.IsValid() && Item->GetTreeItemType() != ITreeItem::HierarchicalLODLevel;  }))
		{
			EmptySelection();

			// Loop over previously retrieve lsit of selected nodes
			StartSelection();

			bool bChanged = false;

			for (FTreeItemPtr SelectedItem : NewSelectedNodes)
			{
				if (SelectedItem.IsValid())
				{
					ITreeItem::TreeItemType Type = SelectedItem->GetTreeItemType();
					switch (Type)
					{
					case ITreeItem::HierarchicalLODLevel:
					{
						// No functionality for select HLOD level items
						break;
					}

					case ITreeItem::HierarchicalLODActor:
					{
						FLODActorItem* ActorItem = (FLODActorItem*)(SelectedItem.Get());
						if (ActorItem->GetParent()->GetTreeItemType() == ITreeItem::HierarchicalLODLevel)
						{
							SelectActorInViewport(ActorItem->LODActor.Get(), 0);
							bChanged = true;
						}
						break;
					}

					case ITreeItem::StaticMeshActor:
					{
						FStaticMeshActorItem* StaticMeshActorItem = (FStaticMeshActorItem*)(SelectedItem.Get());
						SelectActorInViewport(StaticMeshActorItem->StaticMeshActor.Get(), 0);
						bChanged = true;
						break;
					}
					}
				}
			}
			EndSelection(bChanged);
		}		

		SelectedNodes = TreeView->GetSelectedItems();
	}

	void SHLODOutliner::OnOutlinerDoubleClick(FTreeItemPtr TreeItem)
	{
		ITreeItem::TreeItemType Type = TreeItem->GetTreeItemType();
		const bool bActiveViewportOnly = false;
		
		switch (Type)
		{
			case ITreeItem::HierarchicalLODLevel:
			{
				break;
			}

			case ITreeItem::HierarchicalLODActor:
			{
				FLODActorItem* ActorItem = (FLODActorItem*)(TreeItem.Get());
				SelectActorInViewport(ActorItem->LODActor.Get(), 0);
				GEditor->MoveViewportCamerasToActor(*ActorItem->LODActor.Get(), bActiveViewportOnly);
				break;
			}

			case ITreeItem::StaticMeshActor:
			{
				FStaticMeshActorItem* StaticMeshActorItem = (FStaticMeshActorItem*)(TreeItem.Get());
				SelectActorInViewport(StaticMeshActorItem->StaticMeshActor.Get(), 0);
				GEditor->MoveViewportCamerasToActor(*StaticMeshActorItem->StaticMeshActor.Get(), bActiveViewportOnly);
				break;
			}
		}	
	}

	TSharedPtr<SWidget> SHLODOutliner::OnOpenContextMenu()
	{
		if (!CurrentWorld.IsValid())
		{
			return nullptr;
		}

		// Multi-selection support, check if all selected items are of the same type, if so return the appropriate context menu
		auto SelectedItems = TreeView->GetSelectedItems();
		ITreeItem::TreeItemType Type = ITreeItem::Invalid;
		bool bSameType = true;
		for (int32 SelectedIndex = 0; SelectedIndex < SelectedItems.Num(); ++SelectedIndex)
		{
			if (SelectedIndex == 0)
			{
				Type = SelectedItems[SelectedIndex]->GetTreeItemType();
			}
			else
			{
				// Not all of the same types
				if (SelectedItems[SelectedIndex]->GetTreeItemType() != Type)
				{
					bSameType = false; 
					break;
				}
			}
		}

		if (SelectedItems.Num() && bSameType)
		{
			UToolMenus* ToolMenus = UToolMenus::Get();
			static const FName MenuName = "HierarchicalLODOutliner.HLODOutlinerContextMenu";
			if (!ToolMenus->IsMenuRegistered(MenuName))
			{
				ToolMenus->RegisterMenu(MenuName);
			}

			// Build up the menu for a selection
			FToolMenuContext Context;
			UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);
			TreeView->GetSelectedItems()[0]->GenerateContextMenu(Menu, *this);
			return ToolMenus->GenerateWidget(Menu);
		}

		return TSharedPtr<SWidget>();
	}

	void SHLODOutliner::OnItemExpansionChanged(FTreeItemPtr TreeItem, bool bIsExpanded)
	{
		TreeItem->bIsExpanded = bIsExpanded;

		// Expand any children that are also expanded
		for (auto WeakChild : TreeItem->GetChildren())
		{
			auto Child = WeakChild.Pin();
			if (Child->bIsExpanded)
			{
				TreeView->SetItemExpansion(Child, true);
			}
		}
	}

	void SHLODOutliner::StartSelection()
	{
		GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	}

	void SHLODOutliner::EmptySelection()
	{
		GEditor->SelectNone(false, true, true);
		DestroySelectionActors();
	}

	void SHLODOutliner::DestroySelectionActors()
	{		
		SelectedLODActors.Empty();
	}

	void SHLODOutliner::SelectActorInViewport(AActor* Actor, const uint32 SelectionDepth)
	{
		GEditor->SelectActor(Actor, true, false);

		if (Actor->IsA<ALODActor>() && SelectionDepth == 0)
		{
			AddLODActorForBoundsDrawing(Actor);
		}
	}

	void SHLODOutliner::SelectLODActorAndContainedActorsInViewport(ALODActor* LODActor, const uint32 SelectionDepth)
	{
		TArray<AActor*> SubActors;
		HierarchicalLODUtilities->ExtractStaticMeshActorsFromLODActor(LODActor, SubActors);
		for (AActor* SubActor : SubActors)
		{
			SelectActorInViewport(SubActor, SelectionDepth + 1);
		}

		SelectActorInViewport(LODActor, SelectionDepth);
	}

	void SHLODOutliner::SelectContainedActorsInViewport(ALODActor* LODActor, const uint32 SelectionDepth /*= 0*/)
	{
		TArray<AActor*> SubActors;
		HierarchicalLODUtilities->ExtractStaticMeshActorsFromLODActor(LODActor, SubActors);
		for (AActor* SubActor : SubActors)
		{
			SelectActorInViewport(SubActor, SelectionDepth + 1);
		}
	}

	void SHLODOutliner::AddLODActorForBoundsDrawing(AActor* Actor)
	{
		SelectedLODActors.AddUnique(Actor);
	}

	void SHLODOutliner::EndSelection(const bool bChange)
	{
		// Commit selection changes
		GEditor->GetSelectedActors()->EndBatchSelectOperation();

		if (bChange)
		{
			// Fire selection changed event
			GEditor->NoteSelectionChange();
		}
	}

	void SHLODOutliner::OnLevelSelectionChanged(UObject* Obj)
	{		
		USelection* Selection = Cast<USelection>(Obj);
		AActor* SelectedActor = Cast<AActor>(Obj);
		TreeView->ClearSelection();
		DestroySelectionActors();
		if (Selection)
		{
			int32 NumSelected = Selection->Num();
			// TODO changes this for multiple selection support
			for (int32 SelectionIndex = 0; SelectionIndex < NumSelected; ++SelectionIndex)
			{
				AActor* Actor = Cast<AActor>(Selection->GetSelectedObject(SelectionIndex));
				if (Actor)
				{
					auto Item = TreeItemsMap.Find(Actor);
					if (Item)
					{
						SelectItemInTree(*Item);
						TreeView->RequestScrollIntoView(*Item);
					}

					if (Actor->IsA<ALODActor>())
					{
						AddLODActorForBoundsDrawing(Actor);
					}					
				}
			}			
		}
		else if (SelectedActor)
		{
			auto Item = TreeItemsMap.Find(SelectedActor);
			if (Item)
			{
				SelectItemInTree(*Item);

				TreeView->RequestScrollIntoView(*Item);
			}	

			if (SelectedActor->IsA<ALODActor>())
			{
				AddLODActorForBoundsDrawing(SelectedActor);
			}
		}
	}

	void SHLODOutliner::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
	{
		ResetCachedData();
		FullRefresh();
	}

	void SHLODOutliner::OnLevelRemoved(ULevel* InLevel, UWorld* InWorld)
	{
		ResetCachedData();
		FullRefresh();

		SelectedLODActors.RemoveAll([InLevel](const AActor* Actor) { return Actor->GetLevel() == InLevel; });
	}

	void SHLODOutliner::OnLevelActorsAdded(AActor* InActor)
	{
		if (InActor->GetWorld() == CurrentWorld.Get() && !InActor->IsA<AWorldSettings>())
		{
			FullRefresh();
		}	
	}

	void SHLODOutliner::OnLevelActorsRemoved(AActor* InActor)
	{
		if (!InActor->IsA<AWorldSettings>())
		{
			// If actor is a LODActor, make sure it isn't part of the selection array.
			if (InActor->IsA<ALODActor>())
			{
				SelectedLODActors.Remove(InActor);
			}

			// Remove InActor from LOD actor which contains it
			for (TArray<TWeakObjectPtr<ALODActor>>& ActorArray : LODLevelActors)
			{				
				for (TWeakObjectPtr<ALODActor> Actor : ActorArray)
				{
					// Check if actor is not null due to Destroy Actor
					if (Actor.IsValid() && Actor != InActor)
					{
						Actor->CleanSubActorArray();
						Actor->RemoveSubActor(InActor);

						if (Actor->SubActors.Num() == 0)
						{
							HierarchicalLODUtilities->DestroyCluster(Actor.Get());
						}
					}
				}
			}
			FullRefresh();
		} 
	}
	
	void SHLODOutliner::OnMapChange(uint32 MapFlags)
	{
		CurrentWorld = nullptr;
		ResetCachedData();
		FullRefresh();
	}

	void SHLODOutliner::OnNewCurrentLevel()
	{	
		CurrentWorld = nullptr;
		ResetCachedData();
		FullRefresh();
	}

	void SHLODOutliner::OnMapLoaded(const FString& Filename, bool bAsTemplate)
	{
		CurrentWorld = nullptr;
		ResetCachedData();
		FullRefresh();
	}

	void SHLODOutliner::OnHLODActorMovedEvent(const AActor* InActor, const AActor* ParentActor)
	{
		FTreeItemPtr* TreeItem = TreeItemsMap.Find(InActor);
		FTreeItemPtr* ParentItem = TreeItemsMap.Find(ParentActor);
		if (TreeItem && ParentItem)
		{			
			PendingActions.Emplace(FOutlinerAction::MoveItem, *TreeItem, *ParentItem);

			auto CurrentParent = (*TreeItem)->GetParent(); 

			if (CurrentParent.IsValid())
			{
				if (CurrentParent->GetTreeItemType() == ITreeItem::HierarchicalLODActor)
				{
					FLODActorItem* ParentLODActorItem = (FLODActorItem*)CurrentParent.Get();
					if (!ParentLODActorItem->LODActor->HasAnySubActors())
					{
						HierarchicalLODUtilities->DestroyLODActor(ParentLODActorItem->LODActor.Get());
						PendingActions.Emplace(FOutlinerAction::RemoveItem, CurrentParent);
					}
				}
			}
		}
	}

	void SHLODOutliner::OnActorMovedEvent(AActor* InActor)
	{
		if (InActor->IsA<ALODActor>())
		{
			return;
		}

		ALODActor* ParentActor = HierarchicalLODUtilities->GetParentLODActor(InActor);
		if (ParentActor)
		{
			ParentActor->Modify();
		}
	}

	void SHLODOutliner::OnHLODActorAddedEvent(const AActor* InActor, const AActor* ParentActor)
	{
		checkf(InActor != nullptr, TEXT("Invalid InActor found"));
		checkf(ParentActor != nullptr, TEXT("Invalid ParentActor found"));

		FTreeItemPtr* ParentItem = TreeItemsMap.Find(ParentActor);
		if (ParentItem->IsValid())
		{
			const ALODActor* ParentLODActor = Cast<ALODActor>(ParentActor);
			
			FTreeItemPtr* ChildItemPtr = TreeItemsMap.Find(InActor);
			if (ChildItemPtr)
			{
				if (!InActor->IsA<ALODActor>())
				{
					PendingActions.Emplace(FOutlinerAction::MoveItem, *ChildItemPtr, *ParentItem);
				}
				else
				{
					FullRefresh();

					// TODO, handle duplicate entries of same actor in tree view
					/*// Add child item for the new LOD mesh actor
					const ALODActor* LODActor = Cast<ALODActor>(InActor);
					FTreeItemRef ChildItem = MakeShareable(new FLODActorItem(LODActor));
					AllNodes.Add(ChildItem->AsShared());
					(*ParentItem)->AddChild(ChildItem);

					TreeView->RequestTreeRefresh();*/
				}
			}
			else
			{
				// Add child item for the new static mesh actor
				FTreeItemRef ChildItem = MakeShared<FStaticMeshActorItem>(const_cast<AActor*>(InActor));
				AllNodes.Add(ChildItem->AsShared());
				PendingActions.Emplace(FOutlinerAction::AddItem, ChildItem, *ParentItem);
			}

			// Set build flags according to whether or not this LOD actor is dirty 
			LODLevelBuildFlags[ParentLODActor->LODLevel - 1] &= ParentLODActor->IsBuilt(true);
		}
	}

	void SHLODOutliner::OnHLODTransitionScreenSizeChangedEvent()
	{
		if (CurrentWorld.IsValid())
		{	
			const TArray<struct FHierarchicalSimplification>& HierarchicalLODSetups = CurrentWorldSettings->GetHierarchicalLODSetup();
			int32 MaxLODLevel = FMath::Min(HierarchicalLODSetups.Num(), LODLevelTransitionScreenSizes.Num());			
			for (int32 LODLevelIndex = 0; LODLevelIndex < MaxLODLevel; ++LODLevelIndex)
			{
				if (LODLevelTransitionScreenSizes[LODLevelIndex] != HierarchicalLODSetups[LODLevelIndex].TransitionScreenSize)
				{
					LODLevelTransitionScreenSizes[LODLevelIndex] = HierarchicalLODSetups[LODLevelIndex].TransitionScreenSize;
					UpdateDrawDistancesForLODLevel(LODLevelIndex);
				}
			}
		}
	}

	void SHLODOutliner::OnHLODLevelsArrayChangedEvent()
	{
		if (CurrentWorld.IsValid())
		{
			FullRefresh();
		}
	}

	void SHLODOutliner::OnHLODActorRemovedFromClusterEvent(const AActor* InActor, const AActor* ParentActor)
	{
		FTreeItemPtr* TreeItem = TreeItemsMap.Find(InActor);
		FTreeItemPtr* ParentItem = TreeItemsMap.Find(ParentActor);
		if (TreeItem->IsValid() && ParentItem->IsValid())
		{
			checkf((*TreeItem)->GetTreeItemType() == ITreeItem::StaticMeshActor, TEXT("Incorrect InActor"));
			checkf((*ParentItem)->GetTreeItemType() == ITreeItem::HierarchicalLODActor, TEXT("Incorrect ParentActor"));
			PendingActions.Emplace(FOutlinerAction::RemoveItem, *TreeItem);
		}
	}

	void SHLODOutliner::FullRefresh()
	{		
		bNeedsRefresh = true;
	}
	
	const bool SHLODOutliner::UpdateCurrentWorldAndSettings()
	{
		CurrentWorld = nullptr;
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE)
			{
				CurrentWorld = Context.World();
				break;
			}
			else if (Context.WorldType == EWorldType::Editor)
			{
				CurrentWorld = Context.World();
			}
		}	

		if (CurrentWorld.IsValid())
		{
			// Retrieve current world settings
			CurrentWorldSettings = CurrentWorld->GetWorldSettings();
			ensureMsgf(CurrentWorldSettings != nullptr, TEXT("CurrentWorld (%s) does not contain a valid WorldSettings actor"), *CurrentWorld->GetName());
						
			// Update settings view
			SettingsView->SetObject(CurrentWorldSettings);
		}
		

		return (CurrentWorld.IsValid());
	}

	void SHLODOutliner::Populate()
	{
		ResetCachedData();
		const bool bUpdatedWorld = UpdateCurrentWorldAndSettings();
		checkf(bUpdatedWorld == true, TEXT("Could not find UWorld* instance in Engine world contexts"));

		TArray<FTreeItemRef> LevelNodes;
		if (OutlinerEnabled())
		{
			// Iterate over all LOD levels (Number retrieved from world settings) and add Treeview items for them
			const TArray<struct FHierarchicalSimplification>& HierarchicalLODSetups = CurrentWorldSettings->GetHierarchicalLODSetup();
			const uint32 LODLevels = HierarchicalLODSetups.Num();
			
			auto AddHLODLevelItem = [&](const int32 HLODLevelIndex)
			{
				FTreeItemRef LevelItem = MakeShareable(new FLODLevelItem(HLODLevelIndex));

				PendingActions.Emplace(FOutlinerAction::AddItem, LevelItem);
				HLODTreeRoot.Add(LevelItem->AsShared());
				AllNodes.Add(LevelItem->AsShared());

				const int32 RequiredLevelEntries = HLODLevelIndex + 1;
				if (LODLevelActors.Num() < RequiredLevelEntries)
				{
					// Add new HLOD level item to maps and arrays holding cached items
					LODLevelActors.SetNum(RequiredLevelEntries);
					LevelNodes.SetNumZeroed(RequiredLevelEntries);
					LODLevelBuildFlags.SetNum(RequiredLevelEntries);
					LODLevelTransitionScreenSizes.SetNum(RequiredLevelEntries);

					LevelNodes[HLODLevelIndex] = LevelItem->AsShared();
					// Initialize lod level actors/screen size and build flag
					LODLevelBuildFlags[HLODLevelIndex] = true;
					LODLevelTransitionScreenSizes[HLODLevelIndex] = (HierarchicalLODSetups.IsValidIndex(HLODLevelIndex) ? HierarchicalLODSetups[HLODLevelIndex].TransitionScreenSize : 1.0f);
				}

				TreeItemsMap.Add(LevelItem->GetID(), LevelItem);

				// Expand level items by default
				LevelItem->bIsExpanded = true;
			};

			// Add 'known' HLOD level entries
			for (uint32 LODLevelIndex = 0; LODLevelIndex < LODLevels; ++LODLevelIndex)
			{
				AddHLODLevelItem(LODLevelIndex);
			}

			// Loop over all the levels in the current world
			for (ULevel* Level : CurrentWorld->GetLevels())
			{
				// Only handling visible levels (this is to allow filtering the HLOD outliner per level, should change when adding new sortable-column)
				if (Level->bIsVisible)
				{
					for (AActor* Actor : Level->Actors)
					{
						// Only handling LODActors
						if (Actor)
						{
							ALODActor* LODActor = Cast<ALODActor>(Actor);							
							// Add LOD Actor item to the treeview
							if (LODActor)
							{
								// Ad-hoc adding of HLOD level entry
								if (!LODLevelActors.IsValidIndex(LODActor->LODLevel - 1))
								{
									AddHLODLevelItem(LODActor->LODLevel - 1);
								}

								// This is to prevent issues with the sub actor array due to deleted scene actors while the HLOD outliner was closed
								LODActor->CleanSubActorArray();

								// Set LOD parents here TODO remove if not needed anymore QQ
								LODActor->UpdateSubActorLODParents();

								FTreeItemRef Item = MakeShareable(new FLODActorItem(LODActor));
								AllNodes.Add(Item->AsShared());

								PendingActions.Emplace(FOutlinerAction::AddItem, Item, LevelNodes[LODActor->LODLevel - 1]);

								for (AActor* ChildActor : LODActor->SubActors)
								{
									if (ChildActor->IsA<ALODActor>())
									{
										FTreeItemRef ChildItem = MakeShareable(new FLODActorItem(CastChecked<ALODActor>(ChildActor)));
										AllNodes.Add(ChildItem->AsShared());
										Item->AddChild(ChildItem);
									}
									else
									{
										FTreeItemRef ChildItem = MakeShareable(new FStaticMeshActorItem(ChildActor));
										AllNodes.Add(ChildItem->AsShared());

										PendingActions.Emplace(FOutlinerAction::AddItem, ChildItem, Item);
									}
								}

								// Set build flags according to whether or not this LOD actor is dirty 
								LODLevelBuildFlags[LODActor->LODLevel - 1] &= LODActor->IsBuilt(true);
								// Add the actor to it's HLOD levels array
								LODLevelActors[LODActor->LODLevel - 1].Add(LODActor);
							}
						}
					}
				}
			}	

			// Take empty LOD levels into account for the build flags
			for (uint32 LODLevelIndex = 0; LODLevelIndex < LODLevels; ++LODLevelIndex)
			{
				if (LODLevelActors[LODLevelIndex].Num() == 0)
				{
					LODLevelBuildFlags[LODLevelIndex] = true;
				}
			}
		}

		// Request treeview UI item to refresh
		TreeView->RequestTreeRefresh();		

		// Just finished refreshing
		bNeedsRefresh = false;
	}

	void SHLODOutliner::ResetCachedData()
	{
		HLODTreeRoot.Reset();
		TreeItemsMap.Reset();
		LODLevelBuildFlags.Reset();
		LODLevelTransitionScreenSizes.Reset();

		for (auto& ActorArray : LODLevelActors)
		{
			ActorArray.Reset();
		}

		LODLevelActors.Reset();
	}

	TMap<FTreeItemID, bool> SHLODOutliner::GetParentsExpansionState() const
	{
		FParentsExpansionState States;
		for (const auto& Pair : TreeItemsMap)
		{
			if (Pair.Value->GetChildren().Num())
			{
				States.Add(Pair.Key, Pair.Value->bIsExpanded);
			}
		}

		return States;
	}

	void SHLODOutliner::SetParentsExpansionState(const FParentsExpansionState& ExpansionStateInfo) const
	{
		for (const auto& Pair : TreeItemsMap)
		{
			auto& Item = Pair.Value;
			if (Item->GetChildren().Num())
			{
				const bool* bIsExpanded = ExpansionStateInfo.Find(Pair.Key);
				if (bIsExpanded)
				{
					TreeView->SetItemExpansion(Item, *bIsExpanded);
				}
				else
				{
					TreeView->SetItemExpansion(Item, Item->bIsExpanded);
				}
			}
		}
	}

	const bool SHLODOutliner::AddItemToTree(FTreeItemPtr InItem, FTreeItemPtr InParentItem)
	{
		const auto ItemID = InItem->GetID();

		TreeItemsMap.Add(ItemID, InItem);

		if (InParentItem.Get())
		{
			InParentItem->AddChild(InItem->AsShared());
		}		

		return true;
	}

	void SHLODOutliner::MoveItemInTree(FTreeItemPtr InItem, FTreeItemPtr InParentItem)
	{
		auto CurrentParent = InItem->Parent;
		if (CurrentParent.IsValid())
		{
			CurrentParent.Pin()->RemoveChild(InItem->AsShared());
		}

		if (InParentItem.Get())
		{
			InParentItem->AddChild(InItem->AsShared());
		}
	}

	void SHLODOutliner::RemoveItemFromTree(FTreeItemPtr InItem)
	{
		const int32 NumRemoved = TreeItemsMap.Remove(InItem->GetID());

		if (!NumRemoved)
		{
			return;
		}

		auto ParentItem = InItem->GetParent();
		if (ParentItem.IsValid())
		{
			ParentItem->RemoveChild(InItem->AsShared());
		}
	}

	void SHLODOutliner::SelectItemInTree(FTreeItemPtr InItem)
	{
		auto Parent = InItem->GetParent();
		while (Parent.IsValid() && !Parent->bIsExpanded)
		{
			Parent->bIsExpanded = true;
			TreeView->SetItemExpansion(Parent, true);
			Parent = InItem->GetParent();
		}
		TreeView->SetItemSelection(InItem, true);

		TreeView->RequestTreeRefresh();
	}
	
	FReply SHLODOutliner::RetrieveActors()
	{
		bNeedsRefresh = true;
		return FReply::Handled();
	}

	bool SHLODOutliner::OutlinerEnabled() const
	{
		bool bHLODEnabled = CurrentWorldSettings != nullptr;

		if (bHLODEnabled && CurrentWorld.IsValid())
		{
			bHLODEnabled = !HierarchicalLODUtilities->IsWorldUsedForStreaming(CurrentWorld.Get());
		}

		// Disable outliner in PIE
		if (bHLODEnabled && CurrentWorld.IsValid())
		{
			bHLODEnabled = !CurrentWorld->IsPlayInEditor();
		}

		return bHLODEnabled;
	}

	void SHLODOutliner::CloseOpenAssetEditors()
	{
		// Close any asset editors that are looking at our data
		if(CurrentWorld.IsValid())
		{
			for (ULevel* Level : CurrentWorld->GetLevels())
			{
				for (AActor* Actor : Level->Actors)
				{
					if (ALODActor* LODActor = Cast<ALODActor>(Actor))
					{
						if(UHLODProxy* Proxy = LODActor->GetProxy())
						{
							if(UPackage* HLODPackage = Proxy->GetOutermost())
							{
								TArray<UObject*> Objects;
								GetObjectsWithOuter(HLODPackage, Objects);
								for(UObject* PackageObject : Objects)
								{
									GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(PackageObject);
								}
							}
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
