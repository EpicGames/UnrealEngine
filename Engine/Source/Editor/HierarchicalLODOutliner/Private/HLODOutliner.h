// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EditorUndoClient.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "TreeItemID.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class AActor;
class ALODActor;
class AWorldSettings;
class FActiveTimerHandle;
class FDragDropEvent;
class IDetailsView;
class IHierarchicalLODUtilities;
class ITableRow;
class SVerticalBox;
class SWidget;
class ULevel;
class UObject;
class UWorld;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

namespace HLODOutliner
{
	struct FTreeItemID;
	struct ITreeItem;

	typedef TSharedPtr<ITreeItem> FTreeItemPtr;
	typedef TSharedRef<ITreeItem> FTreeItemRef;

	/**
	* Outliner action class used for making changing to the Outliner's treeview 
	*/	
	struct FOutlinerAction
	{
		enum ActionType
		{
			AddItem,
			RemoveItem,
			MoveItem
		};

		FOutlinerAction(ActionType InType, FTreeItemPtr InItem) : Type(InType), Item(InItem) {};
		FOutlinerAction(ActionType InType, FTreeItemPtr InItem, FTreeItemPtr InParentItem) : Type(InType), Item(InItem), ParentItem( InParentItem) {};

		ActionType Type;
		FTreeItemPtr Item;
		FTreeItemPtr ParentItem;
	};

	/**
	* Implements the profiler window.
	*/
	class SHLODOutliner : public SCompoundWidget, public FNotifyHook, public FEditorUndoClient
	{
	typedef STreeView<FTreeItemPtr> SHLODTree;
	friend struct FLODActorItem;
	friend struct FHLODTreeWidgetItem;
	friend struct FStaticMeshActorItem;
	friend struct FLODLevelItem;
	friend class SHLODWidgetItem;

	public:
		/** Default constructor. */
		SHLODOutliner();

		/** Virtual destructor. */
		virtual ~SHLODOutliner();

		SLATE_BEGIN_ARGS(SHLODOutliner){}
		SLATE_END_ARGS()

		/**
		* Constructs this widget.
		*/
		void Construct(const FArguments& InArgs);

		/** Create the panel's forced HLOD level viewer */
		TSharedRef<SWidget> CreateForcedViewWidget();

		/** Creates the panel's main button widget rows */
		TSharedRef<SWidget> CreateMainButtonWidgets();

		/** Creates the panel's button widgets for cluster operations */
		TSharedRef<SWidget> CreateClusterButtonWidgets();

		/** Creates the panel's Tree view widget*/
		TSharedRef<SHLODTree> CreateTreeviewWidget();
		
		/** Initializes and creates the settings view */
		void CreateSettingsView();

		//~ Begin SCompoundWidget Interface
		virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
		virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)  override;
		//~ End SCompoundWidget Interface

		//~ Begin FEditorUndoClient Interface
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
		// End of FEditorUndoClient	

		/** Button handlers */
		FText GetForceBuildText() const;
		FText GetForceBuildToolTip() const;
		FText GetBuildText() const;
		FReply HandleDeleteHLODs();
		bool CanDeleteHLODs() const;
		FReply HandlePreviewHLODs();
		FReply RetrieveActors();
		FReply HandleBuildLODActors();
		bool CanBuildLODActors() const;
		FText GetBuildLODActorsTooltipText() const;
		FReply HandleForceBuildLODActors();
		FReply HandleForceRefresh();
		FReply HandleSaveAll();
		FText GetGenerateClustersText() const;
		FText GetGenerateClustersTooltip() const;
		FText GetRegenerateClustersText() const;
		FText GetRegenerateClustersTooltip() const;
		/** End button handlers */

	private:
		/** Registers all the callback delegates required for keeping the treeview sync */
		void RegisterDelegates();

		/** De-registers all the callback delegates required for keeping the treeview sync */
		void DeregisterDelegates();

		/** Builds the toolbar */
		TSharedRef<SWidget> MakeToolBar();

		/** Callbacks to make interaction from UI uniform vs. UI updates*/
		FReply GenerateClustersFromUI();
		FReply GenerateProxyMeshesFromUI();
		FReply BuildClustersAndMeshesFromUI();

		/** Helper function used to determine whether there are any LOD actors present */
		bool HasHLODActors() const;

	protected:
		/** Forces viewing the mesh of a Cluster (LODActor) */
		void ForceViewLODActor();

		/**
		* Returns whether or not all HLODs in the level are build
		*
		* @return bool
		*/
		bool AreHLODsBuild() const;
		
		/**
		* Returns FText with information about current forced HLOD level "None", "1" etc.
		*
		* @return FText
		*/
		FText HandleForceLevelText() const;

		/** Build the forced level widget */
		TSharedRef<SWidget> GetForceLevelMenuContent() const;

		/**
		* Restores the forced viewing state for the given LOD levle
		*
		* @param LODLevel - LOD level to force
		*/
		void RestoreForcedLODLevel(int32 LODLevel);

		/**
		* Forces LODActors within the given LODLevel to show their meshes (other levels hide theirs)
		*
		* @param LODLevel - LOD level to force
		*/
		void SetForcedLODLevel(int32 LODLevel);

		/** Resets the forced LOD level */
		void ResetLODLevelForcing();

		/** Creates a Hierarchical LOD Volume for the given LODActorItem, volume bounds correspond to those of the LODActor's SubActors */
		void CreateHierarchicalVolumeForActor();
	protected:
		/** Builds the HLOD mesh for the given ALODActor (cluster) */
		void BuildLODActor();	

		/** Rebuilds the HLOD mesh for the given ALODActor (cluster) */
		void RebuildLODActor();

		/** Select the LODActor in the Editor Viewport */
		void SelectLODActor();

		/** Deletes a cluster (LODActor) */
		void DeleteCluster();

		/** Selects the contained actors (SubActors) for a specific LODActor */
		void SelectContainedActors();

		/** Removes the given StaticMeshActor from its parent's (ALODActor) sub-actors array */
		void RemoveStaticMeshActorFromCluster();
		
		/* Removes the given StaticMeshActor from its parent's (ALODActor) sub-actors array and excludes it from cluster generation */
		void ExcludeFromClusterGeneration();		

		/** Removes the given LODActor from its parent's (ALODActor) sub-actors array */
		void RemoveLODActorFromCluster();

		/** Creates a cluster from a set of actors for the given lod level index */
		void CreateClusterFromActors(const TArray<AActor*>& Actors, uint32 LODLevelIndex);
		
		/**
		* Updates the DrawDistance value for all the LODActors with the given LODLevelIndex
		*
		* @param LODLevelIndex -
		*/
		void UpdateDrawDistancesForLODLevel(const uint32 LODLevelIndex);
		
		/**
		* Removes LODActors within the given HLODLevel
		*
		* @param LODLevelIndex -		
		*/
		void RemoveLODLevelActors(const int32 HLODLevelIndex);

		/** Handle splitting horizontally or vertically based on dimensions */
		int32 GetSpitterWidgetIndex() const;
	protected:
		/** Tree view callbacks */

		/**
		* Generates a Tree view row for the given Tree view Node
		*
		* @param InReflectorNode - Node to generate a row for
		* @param OwnerTable - Owning table of the InReflectorNode
		* @return TSharedRef<ITableRow>
		*/
		TSharedRef<ITableRow> OnOutlinerGenerateRow(FTreeItemPtr InReflectorNode, const TSharedRef<STableViewBase>& OwnerTable);
	
		/**
		* Treeview callback for retrieving the children of a specific TreeItem
		*
		* @param InParent - Parent item to return the children from
		* @param OutChildren - InOut array for the children
		*/
		void OnOutlinerGetChildren(FTreeItemPtr InParent, TArray<FTreeItemPtr>& OutChildren);

		/**
		* Handles the event fired when selection with the HLOD Tree view changes
		*
		* @param TreeItem - Selected node(s)
		* @param SelectionInfo - Type of selection change
		*/
		void OnOutlinerSelectionChanged(FTreeItemPtr TreeItem, ESelectInfo::Type /*SelectionInfo*/);
	
		/**
		* Handles double click events from the HLOD Tree view
		*
		* @param TreeItem - Node which was double-clicked
		*/
		void OnOutlinerDoubleClick(FTreeItemPtr TreeItem);

		/** Open a context menu for this scene outliner */
		TSharedPtr<SWidget> OnOpenContextMenu();

		/**
		* Handles item expansion events from the HLOD tree view
		*
		* @param TreeItem - Item which expansion state was changed
		* @param bIsExpanded - New expansion state
		*/
		void OnItemExpansionChanged(FTreeItemPtr TreeItem, bool bIsExpanded);
		
		/** End of Tree view callbacks */

	private:
		/** Starts the Editor selection batch */
		void StartSelection();

		/** Empties the current Editor selection */
		void EmptySelection();

		/** Destroys the created selection actors */
		void DestroySelectionActors();		

		/**
		* Selects an Actor in the Editor viewport
		*
		* @param Actor - AActor to select inside the viewport
		* @param SelectionDepth - (recursive)
		*/
		void SelectActorInViewport(AActor* Actor, const uint32 SelectionDepth = 0);

		/**
		* Selects actors and sub-actors and the given LODActor
		*
		* @param LODActor - Actor to select + subactors
		* @param SelectionDepth - (recursive)
		*/
		void SelectLODActorAndContainedActorsInViewport(ALODActor* LODActor, const uint32 SelectionDepth = 0);

		/**
		* Selects actors and sub-actors for the given LODActor
		*
		* @param LODActor - Actor to select + subactors
		* @param SelectionDepth - (recursive)
		*/
		void SelectContainedActorsInViewport(ALODActor* LODActor, const uint32 SelectionDepth = 0);

		/**
		* Creates a ASelectionActor for the given actor, "procedurally" drawing its bounds
		*
		* @param Actor - Actor to create the SelectionActor for
		* @return UDrawSphereComponent*
		*/
		void AddLODActorForBoundsDrawing(AActor* Actor);

		/** Ends the Editor selection batch, bChange determines whether or not there was an actual change and call NoteSelectionChange */
		void EndSelection(const bool bChange);

	protected:
		/** Broadcast event delegates */

		/** Called by USelection::SelectionChangedEvent delegate when the level's selection changes */
		void OnLevelSelectionChanged(UObject* Obj);

		/** Called by the engine when a level is added to the world. */
		void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);

		/** Called by the engine when a level is removed from the world. */
		void OnLevelRemoved(ULevel* InLevel, UWorld* InWorld);

		/** Called by the engine when an actor is added to the world. */
		void OnLevelActorsAdded(AActor* InActor);

		/** Called by the engine when an actor is remove from the world. */
		void OnLevelActorsRemoved(AActor* InActor);
		
		/** Called when the map has changed*/
		void OnMapChange(uint32 MapFlags);

		/** Called when the current level has changed */
		void OnNewCurrentLevel();

		/** Called when a new map is being loaded */
		void OnMapLoaded(const FString&  Filename, bool bAsTemplate);

		/** Called when a HLODActor is moved between clusters */
		void OnHLODActorMovedEvent(const AActor* InActor, const AActor* ParentActor);

		/** Called when an Actor is moved inside of the level */
		void OnActorMovedEvent(AActor* InActor);

		/** Called when and HLODActor is added to the level */
		void OnHLODActorAddedEvent(const AActor* InActor, const AActor* ParentActor);

		/** Called when a DrawDistance value within WorldSettings changed */
		void OnHLODTransitionScreenSizeChangedEvent();

		/** Called when the HLOD Levels array within WorldSettings changed */
		void OnHLODLevelsArrayChangedEvent();

		/** Called when an Actor is removed from a cluster */
		void OnHLODActorRemovedFromClusterEvent(const AActor* InActor, const AActor* ParentActor);

		/** End of Broadcast event delegates */

		/** Callback function used to check if Hierarchical LOD functionality is enabled in the current world settings */
		bool OutlinerEnabled() const;

		/** Called when a PIE session is beginning */
		void OnBeginPieEvent(bool bIsSimulating);

		/** Called when a PIE session is ending */
		void OnEndPieEvent(bool bIsSimulating);

	private:
		/** Tells the scene outliner that it should do a full refresh, which will clear the entire tree and rebuild it from scratch. */
		void FullRefresh();

		/** Retrieves and updates the current world and world settings pointers (returns whether or not a world was found) */
		const bool UpdateCurrentWorldAndSettings();

		/** Populates the HLODTreeRoot array and consequently the Treeview */
		void Populate();

		/** Clears and resets all arrays and maps containing cached/temporary data */
		void ResetCachedData();

		/** Structure containing information relating to the expansion state of parent items in the tree */
		typedef TMap<FTreeItemID, bool> FParentsExpansionState;

		/** Gets the current expansion state of parent items */
		TMap<FTreeItemID, bool> GetParentsExpansionState() const;

		/** Updates the expansion state of parent items after a repopulate, according to the previous state */
		void SetParentsExpansionState(const FParentsExpansionState& ExpansionStateInfo) const;

		/**
		* Adds a new Treeview item
		*
		* @param InItem - Item to add
		* @param InParentItem - Optional parent item to add it to
		* @return const bool
		*/
		const bool AddItemToTree(FTreeItemPtr InItem, FTreeItemPtr InParentItem);

		/**
		* Moves a TreeView item around
		*
		* @param InItem - Item to move
		* @param InParentItem - New parent for InItem to move to
		*/
		void MoveItemInTree(FTreeItemPtr InItem, FTreeItemPtr InParentItem);

		/**
		* Removes a TreeView item
		*
		* @param InItem - Item to remove
		*/
		void RemoveItemFromTree(FTreeItemPtr InItem);

		/**
		* Selects a Treeview item
		*
		* @param InItem - Item to select
		*/
		void SelectItemInTree(FTreeItemPtr InItem);

		/** Timer to update cached 'needs build' flag */
		EActiveTimerReturnType UpdateNeedsBuildFlagTimer(double InCurrentTime, float InDeltaTime);

		/** Closes asset editors showing HLOD data */
		void CloseOpenAssetEditors();

	private:
		/** Whether or not we need to do a refresh of the Tree view*/
		bool bNeedsRefresh;

		/** World instance we are currently representing/mirroring in the panel */
		TWeakObjectPtr<UWorld> CurrentWorld;

		/** World settings found in CurrentWorld */
		AWorldSettings* CurrentWorldSettings;

		/** Tree view nodes */
		TArray<FTreeItemPtr> HLODTreeRoot;
		/** Currently selected Tree view nodes*/
		TArray<FTreeItemPtr> SelectedNodes;
		/** HLOD Treeview widget*/
		TSharedPtr<SHLODTree> TreeView;
		/** Property viewing widget */
		TSharedPtr<IDetailsView> SettingsView;
		/** Content panel widget */
		TSharedPtr<SVerticalBox> MainContentPanel;
		/** Attribute determining if the outliner UI is enabled*/
		TAttribute<bool> EnabledAttribute;

		/** Map containing all the nodes with their corresponding keys */
		TMultiMap<FTreeItemID, FTreeItemPtr> TreeItemsMap;

		/** Array of pending OutlinerActions */
		TArray<FOutlinerAction> PendingActions;

		/** Array containing all the nodes */
		TArray<FTreeItemPtr> AllNodes;
	
		/** Array of currently selected LODActors */		
		TArray<AActor*> SelectedLODActors;

		/** Currently forced LOD level*/
		int32 ForcedLODLevel;

		/** Array with flags for each LOD level (whether or not all their Clusters/LODActors have their meshes built) */
		TArray<bool> LODLevelBuildFlags;
		/** Array of LODActors/Cluster per LOD level*/
		TArray<TArray<TWeakObjectPtr<ALODActor>>> LODLevelActors;
		/** Array of TransitionScreenSizes for each LOD Level*/
		TArray<float> LODLevelTransitionScreenSizes;

		/** Cached pointer to HLOD utilities */
		IHierarchicalLODUtilities* HierarchicalLODUtilities;

		/** Update active timer handle */
		TSharedPtr<FActiveTimerHandle> ActiveTimerHandle;

		/** Cached flag to see if we need to generate meshes (any actors are dirty) */
		bool bCachedNeedsBuild;

		/** Whether to arrange the main UI horizontally or vertically */
		bool bArrangeHorizontally;
	};
};
