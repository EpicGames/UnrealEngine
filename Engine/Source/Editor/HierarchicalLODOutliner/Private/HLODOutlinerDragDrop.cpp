// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLODOutlinerDragDrop.h"

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "IHierarchicalLODUtilities.h"
#include "ITreeItem.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

HLODOutliner::FDragDropPayload::FDragDropPayload()
{
	LODActors = TArray<TWeakObjectPtr<AActor>>();
	StaticMeshActors = TArray<TWeakObjectPtr<AActor>>();
	bSceneOutliner = false;
}

EClusterGenerationError HLODOutliner::FDragDropPayload::ParseDrag(const FDragDropOperation& Operation)
{
	EClusterGenerationError ErrorValue = EClusterGenerationError::None;

	if (Operation.IsOfType<FHLODOutlinerDragDropOp>())
	{
		bSceneOutliner = false;

		const auto& OutlinerOp = static_cast<const FHLODOutlinerDragDropOp&>(Operation);

		if (OutlinerOp.StaticMeshActorOp.IsValid())
		{
			StaticMeshActors = OutlinerOp.StaticMeshActorOp->Actors;
		}

		if (OutlinerOp.LODActorOp.IsValid())
		{
			LODActors = OutlinerOp.LODActorOp->Actors;
		}

		ErrorValue |= EClusterGenerationError::ValidActor;
	}
	else if (Operation.IsOfType<FActorDragDropGraphEdOp>())
	{
		bSceneOutliner = true;

		int32 NumInvalidActors = 0;
		const auto& ActorOp = static_cast<const FActorDragDropGraphEdOp&>(Operation);
		for (auto& ActorPtr : ActorOp.Actors)
		{
			AActor* Actor = ActorPtr.Get();
			FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
			IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

			EClusterGenerationError ClusterGenerationResult = Utilities->ShouldGenerateCluster(Actor, INDEX_NONE);
			ErrorValue |= ClusterGenerationResult;
			if ((ClusterGenerationResult & EClusterGenerationError::ValidActor) != EClusterGenerationError::None)
			{
				if (!StaticMeshActors.IsSet())
				{
					StaticMeshActors = TArray<TWeakObjectPtr<AActor>>();
				}

				StaticMeshActors->Add(Actor);
			}
			else
			{
				++NumInvalidActors;
			}
		}
	}

	return ErrorValue;
}

TSharedPtr<FDragDropOperation> HLODOutliner::CreateDragDropOperation(const TArray<FTreeItemPtr>& InTreeItems)
{
	FDragDropPayload DraggedObjects;
	for (const auto& Item : InTreeItems)
	{
		Item->PopulateDragDropPayload(DraggedObjects);
	}

	if (DraggedObjects.LODActors.IsSet() || DraggedObjects.StaticMeshActors.IsSet())
	{
		TSharedPtr<FHLODOutlinerDragDropOp> OutlinerOp = MakeShareable(new FHLODOutlinerDragDropOp(DraggedObjects));
		OutlinerOp->Construct();
		return OutlinerOp;
	}
	
	return nullptr;
}

HLODOutliner::FHLODOutlinerDragDropOp::FHLODOutlinerDragDropOp(const FDragDropPayload& DraggedObjects)
	: OverrideText()
	, OverrideIcon(nullptr)
{
	if (DraggedObjects.StaticMeshActors)
	{
		StaticMeshActorOp = MakeShareable(new FActorDragDropOp);
		StaticMeshActorOp->Init(DraggedObjects.StaticMeshActors.GetValue());
	}
	
	if (DraggedObjects.LODActors)
	{
		LODActorOp = MakeShareable(new FActorDragDropOp);
		LODActorOp->Init(DraggedObjects.LODActors.GetValue());
	}
}

TSharedPtr<SWidget> HLODOutliner::FHLODOutlinerDragDropOp::GetDefaultDecorator() const
{
	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	VerticalBox->AddSlot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Visibility(this, &FHLODOutlinerDragDropOp::GetOverrideVisibility)
			.Content()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 3.0f, 0.0f)
				[
					SNew(SImage)
					.Image(this, &FHLODOutlinerDragDropOp::GetOverrideIcon)
				]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(this, &FHLODOutlinerDragDropOp::GetOverrideText)
					]
			]
		];

	if (LODActorOp.IsValid())
	{
		auto Content = LODActorOp->GetDefaultDecorator();
		if (Content.IsValid())
		{
			Content->SetVisibility(TAttribute<EVisibility>(this, &FHLODOutlinerDragDropOp::GetDefaultVisibility));
			VerticalBox->AddSlot()[Content.ToSharedRef()];
		}
	}

	if (StaticMeshActorOp.IsValid())
	{
		auto Content = StaticMeshActorOp->GetDefaultDecorator();
		if (Content.IsValid())
		{
			Content->SetVisibility(TAttribute<EVisibility>(this, &FHLODOutlinerDragDropOp::GetDefaultVisibility));
			VerticalBox->AddSlot()[Content.ToSharedRef()];
		}
	}

	return VerticalBox;
}

EVisibility HLODOutliner::FHLODOutlinerDragDropOp::GetOverrideVisibility() const
{
	return OverrideText.IsEmpty() && OverrideIcon == nullptr ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility HLODOutliner::FHLODOutlinerDragDropOp::GetDefaultVisibility() const
{
	return OverrideText.IsEmpty() && OverrideIcon == nullptr ? EVisibility::Visible : EVisibility::Collapsed;
}
