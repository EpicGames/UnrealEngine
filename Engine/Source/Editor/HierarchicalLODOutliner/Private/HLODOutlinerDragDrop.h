// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HierarchicalLODType.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FActorDragDropOp;
class SWidget;
class UWorld;
struct FSlateBrush;

enum class EClusterGenerationError : uint32;

namespace HLODOutliner
{
	struct ITreeItem;

	typedef TSharedPtr<ITreeItem> FTreeItemPtr;

	/** Consilidated drag/drop information parsed for the HLOD outliner */
	struct FDragDropPayload
	{
		/** Default constructor, resulting in unset contents */
		FDragDropPayload();

		/** Populate this payload from an array of tree items */
		template<typename TItem>
		FDragDropPayload(const TArray<TItem>& InDraggedItems)
		{
			for (const auto& Item : InDraggedItems)
			{
				Item->PopulateDragDropPayload(*this);
			}
		}

		/** Optional array of dragged LOD actors */
		TOptional<TArray<TWeakObjectPtr<AActor>>> LODActors;

		/** OPtional array of dragged static mesh actors */
		TOptional<TArray<TWeakObjectPtr<AActor>>> StaticMeshActors;

		/** Flag whether or not this is a drop coming from the SceneOutliner or within the HLOD Outliner */
		bool bSceneOutliner;

		/** World instance that is being used for the HLOD Outliner */
		UWorld* OutlinerWorld;

		/**
		*	Parse a drag operation into our list of actors and folders
		*	@return Error type None if the operation is viable for the HLOD outliner to process, Error type X otherwise
		*/
		EClusterGenerationError ParseDrag(const FDragDropOperation& Operation);
	};

	/** Construct a new Drag and drop operation for a scene outliner */
	TSharedPtr<FDragDropOperation> CreateDragDropOperation(const TArray<FTreeItemPtr>& InTreeItems);

	/** A drag/drop operation that was started from the scene outliner */
	struct FHLODOutlinerDragDropOp : public FDragDropOperation
	{
		enum ToolTipIconType
		{
			ToolTip_Compatible,
			ToolTip_Incompatible,
			ToolTip_Warning,
			ToolTip_MultipleSelection_Compatible,
			ToolTip_MultipleSelection_Incompatible,
			ToolTip_MultiSelection_Warning			
		};

		DRAG_DROP_OPERATOR_TYPE(FHLODOutlinerDragDropOp, FDragDropOperation);

		FHLODOutlinerDragDropOp(const FDragDropPayload& DraggedObjects);

		using FDragDropOperation::Construct;

		/** Actor drag operation */
		TSharedPtr<FActorDragDropOp>	StaticMeshActorOp;
		TSharedPtr<FActorDragDropOp>	LODActorOp;

		void ResetTooltip()
		{
			OverrideText = FText();
			OverrideIcon = nullptr;
		}

		void SetTooltip(FText InOverrideText, const FSlateBrush* InOverrideIcon)
		{
			OverrideText = InOverrideText;
			OverrideIcon = InOverrideIcon;
		}

		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

	private:

		EVisibility GetOverrideVisibility() const;
		EVisibility GetDefaultVisibility() const;

		FText OverrideText;
		FText GetOverrideText() const { return OverrideText; }

		const FSlateBrush* OverrideIcon;
		const FSlateBrush* GetOverrideIcon() const { return OverrideIcon; }
	};

	/** Struct used for validation of a drag/drop operation in the scene outliner */
	struct FDragValidationInfo
	{
		/** The tooltip type to display on the operation */
		FHLODOutlinerDragDropOp::ToolTipIconType TooltipIconType;

		/** The tooltip text to display on the operation */
		FText ValidationText;

		EHierarchicalLODActionType ActionType;

		/** Construct this validation information out of a tooltip type and some text */
		FDragValidationInfo(const EHierarchicalLODActionType InActionType, const FHLODOutlinerDragDropOp::ToolTipIconType InTooltipIconType, const FText InValidationText)
			: TooltipIconType(InTooltipIconType)
			, ValidationText(InValidationText)
			, ActionType(InActionType)
		{			
		}

		/** Return a generic invalid result */
		static FDragValidationInfo Invalid()
		{
			return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, FText());
		}

		/** @return true if this operation is valid, false otherwise */
		bool IsValid() const
		{
			switch (TooltipIconType)
			{
			case FHLODOutlinerDragDropOp::ToolTip_Compatible:
			case FHLODOutlinerDragDropOp::ToolTip_MultipleSelection_Compatible:
				return true;
			default:
				return false;
			}
		}
	};
};
