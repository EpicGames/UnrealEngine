// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLODTreeWidgetItem.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DragAndDrop/ActorDragDropGraphEdOp.h"
#include "HAL/Platform.h"
#include "HLODOutlinerDragDrop.h"
#include "HierarchicalLODType.h"
#include "IHierarchicalLODUtilities.h"
#include "ITreeItem.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "LODActorItem.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "HLODTreeWidgetItem"

namespace HLODOutliner
{
	static void UpdateOperationDecorator(const FDragDropEvent& Event, const FDragValidationInfo& ValidationInfo)
	{
		const FSlateBrush* Icon = ValidationInfo.IsValid() ? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")) : FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

		FDragDropOperation* Operation = Event.GetOperation().Get();
		if (Operation)
		{
			if (Operation->IsOfType<FHLODOutlinerDragDropOp>())
			{
				auto* OutlinerOp = static_cast<FHLODOutlinerDragDropOp*>(Operation);
				OutlinerOp->SetTooltip(ValidationInfo.ValidationText, Icon);
			}
			else if (Operation->IsOfType<FActorDragDropGraphEdOp>())
			{
				auto* ActorOp = static_cast<FActorDragDropGraphEdOp*>(Operation);

				if (ValidationInfo.IsValid())
				{
					ActorOp->SetToolTip(FActorDragDropGraphEdOp::ToolTip_CompatibleGeneric, ValidationInfo.ValidationText );
				}
				else
				{ 
					ActorOp->SetToolTip(FActorDragDropGraphEdOp::ToolTip_IncompatibleGeneric, ValidationInfo.ValidationText);
				}				
			}
		}
	}

	static void ResetOperationDecorator(const FDragDropEvent& Event)
	{
		FDragDropOperation* Operation = Event.GetOperation().Get();
		if (Operation)
		{
			if (Operation->IsOfType<FActorDragDropGraphEdOp>())
			{
				static_cast<FActorDragDropGraphEdOp*>(Operation)->ResetToDefaultToolTip();
			}
		}
	}

	static FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr<STableViewBase> Table)
	{
		auto TablePtr = Table.Pin();
		if (TablePtr.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			auto TreeView = (STreeView< FTreeItemPtr >*)TablePtr.Get();

			auto Operation = HLODOutliner::CreateDragDropOperation(TreeView->GetSelectedItems());
			if (Operation.IsValid())
			{
				return FReply::Handled().BeginDragDrop(Operation.ToSharedRef());
			}
		}

		return FReply::Unhandled();
	}
	
	FReply HandleDrop(TWeakPtr<STableViewBase> Widget, const FDragDropEvent& DragDropEvent, IDropTarget& DropTarget, FDragValidationInfo& ValidationInfo, SHLODWidgetItem* DroppedWidget, bool bApplyDrop)
	{
		FDragDropPayload DraggedObjects;
		// Validate now to make sure we don't doing anything we shouldn't
		EClusterGenerationError ErrorValue = DraggedObjects.ParseDrag(*DragDropEvent.GetOperation());

		if (ErrorValue != EClusterGenerationError::ValidActor)
		{
			// Invalid selection
			FString ErrorString("Selection contains:");
			if ((ErrorValue & EClusterGenerationError::AlreadyClustered) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("ActorsAlreadyClustered", "\n- Already clustered Actor(s)").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::InvalidActor) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("InvalidActorsSelected", "\n- Invalid Actor(s)").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::ActorHiddenInGame) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("ActorsHiddenInGame", "\n- Actor(s) which are hidden In Game").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::ExcludedActor) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("ActorsExcludedFromHLOD", "\n- Actor(s) which are set to be excluded from HLOD generation").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::LODActor) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("LODActorsSelected", "\n- LOD Actor(s)").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::ActorTooSmall) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("InvisibleBoundActors", "\n- Actor(s) with invisible Bounds").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::ComponentHiddenInGame) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("ComponentsHiddenInGame", "\n- Actor(s) with Components set to be hidden In Game").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::MoveableComponent) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("MoveableActors", "\n- Actor(s) with Moveable Components").ToString();
			}
			
			if ((ErrorValue & EClusterGenerationError::ExcludedComponent) != EClusterGenerationError::None)
			{
				ErrorString += LOCTEXT("ComponentsExcluded", "\n- Actor(s) with Components set to be excluded from HLOD generation").ToString();
			}

			if ((ErrorValue & EClusterGenerationError::ValidActor) != EClusterGenerationError::None)
			{
				// Warning(s)
				ValidationInfo = DropTarget.ValidateDrop(DraggedObjects);
				FTextBuilder TextBuilder;
				TextBuilder.AppendLineFormat(FTextFormat::FromString("{0} - (Warning) {1}"), ValidationInfo.ValidationText, FText::FromString(ErrorString));
				ValidationInfo.ValidationText = TextBuilder.ToText();
			}
			else
			{
				// Error
				ValidationInfo = FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, FText::FromString(TEXT("(Error) ") + ErrorString));
				return FReply::Unhandled();
			}
		}
		else
		{
			ValidationInfo = DropTarget.ValidateDrop(DraggedObjects);
		}


		if (!ValidationInfo.IsValid())
		{
			// Return handled here to stop anything else trying to handle it - the operation is invalid as far as we're concerned
			return FReply::Handled();
		}

		if (bApplyDrop)
		{
			DraggedObjects.OutlinerWorld = DroppedWidget->GetWorld();
			DropTarget.OnDrop(DraggedObjects, ValidationInfo, Widget.Pin().ToSharedRef());
		}

		return FReply::Handled();
	}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void SHLODWidgetItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
	{
		this->TreeItem = InArgs._TreeItemToVisualize.Get();
		this->Outliner = InArgs._Outliner;
		this->World = InArgs._World;
		check(TreeItem);

		WeakTableViewBase = InOwner;

		auto Args = FSuperRowType::FArguments()
			.Padding(1.0f)
			.OnDragDetected_Static(HLODOutliner::OnDragDetected, TWeakPtr<STableViewBase>(InOwner));

		SMultiColumnTableRow< FTreeItemPtr >::Construct(Args, InOwner);
	}


	TSharedRef<SWidget> SHLODWidgetItem::GenerateWidgetForColumn(const FName& ColumnName)
	{
		const float DefaultColumnHorizontalPadding = 24.0f;
		const float LastColumnHorizontalPadding = 8.0f;

		if (ColumnName == TEXT("SceneActorName"))
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SExpanderArrow, SharedThis(this))
					.IndentAmount(20)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &SHLODWidgetItem::GetItemDisplayString)
					.ColorAndOpacity(this, &SHLODWidgetItem::GetTint)
				];
		}		
		else if (ColumnName == TEXT("RawTriangleCount") && TreeItem->GetTreeItemType() == ITreeItem::HierarchicalLODActor)
		{
			FLODActorItem* Item = static_cast<FLODActorItem*>(TreeItem);

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(DefaultColumnHorizontalPadding, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Item, &FLODActorItem::GetRawNumTrianglesAsText)
					.ColorAndOpacity(this, &SHLODWidgetItem::GetTint)
				];
		}
		else if (ColumnName == TEXT("ReducedTriangleCount") && TreeItem->GetTreeItemType() == ITreeItem::HierarchicalLODActor)
		{
			FLODActorItem* Item = static_cast<FLODActorItem*>(TreeItem);

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(DefaultColumnHorizontalPadding, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Item, &FLODActorItem::GetReducedNumTrianglesAsText)
					.ColorAndOpacity(this, &SHLODWidgetItem::GetTint)
				];
		}
		else if (ColumnName == TEXT("ReductionPercentage") && TreeItem->GetTreeItemType() == ITreeItem::HierarchicalLODActor)
		{
			FLODActorItem* Item = static_cast<FLODActorItem*>(TreeItem);

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(DefaultColumnHorizontalPadding, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Item, &FLODActorItem::GetReductionPercentageAsText)
					.ColorAndOpacity(this, &SHLODWidgetItem::GetTint)
				];
		}
		else if (ColumnName == TEXT("Level") && TreeItem->GetTreeItemType() == ITreeItem::HierarchicalLODActor)
		{
			FLODActorItem* Item = static_cast<FLODActorItem*>(TreeItem);

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(LastColumnHorizontalPadding, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Item, &FLODActorItem::GetLevelAsText)
					.ColorAndOpacity(this, &SHLODWidgetItem::GetTint)
				];
		}
		else
		{
			return SNullWidget::NullWidget;
		}
	}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	FText SHLODWidgetItem::GetItemDisplayString() const
	{
		return FText::FromString(TreeItem->GetDisplayString());
	}

	FSlateColor SHLODWidgetItem::GetTint() const
	{
		return TreeItem->GetTint();
	}

	void SHLODWidgetItem::OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent)
	{
		if (TreeItem)
		{
			FDragValidationInfo ValidationInfo = FDragValidationInfo::Invalid();

			HLODOutliner::HandleDrop(WeakTableViewBase, DragDropEvent, *TreeItem, ValidationInfo, this, false);
			UpdateOperationDecorator(DragDropEvent, ValidationInfo);
		}
	}

	void SHLODWidgetItem::OnDragLeave(FDragDropEvent const& DragDropEvent)
	{
		ResetOperationDecorator(DragDropEvent);
	}

	FReply SHLODWidgetItem::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		return FReply::Handled();
	}

	FReply SHLODWidgetItem::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		if (TreeItem)
		{
			FDragValidationInfo ValidationInfo = FDragValidationInfo::Invalid();
			return HLODOutliner::HandleDrop(WeakTableViewBase, DragDropEvent, *TreeItem, ValidationInfo, this, true);
		}

		return FReply::Unhandled();
	}	
};

#undef LOCTEXT_NAMESPACE // "HLODTreeWidgetItem"
