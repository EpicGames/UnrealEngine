// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Delegates/IDelegateInstance.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FDragDropEvent;
class STableViewBase;
class SWidget;
class UWorld;
struct FGeometry;

namespace HLODOutliner
{
	struct ITreeItem;

	typedef TSharedPtr<ITreeItem> FTreeItemPtr;
	class SHLODOutliner;

	/**
	* Widget that visualizes the contents of a FReflectorNode.
	*/
	class SHLODWidgetItem
		: public SMultiColumnTableRow < FTreeItemPtr >
	{
	public:

		SLATE_BEGIN_ARGS(SHLODWidgetItem)
			: _TreeItemToVisualize(),
			_Outliner(),
			_World()
		{ }

		SLATE_ARGUMENT(FTreeItemPtr, TreeItemToVisualize)
		SLATE_ARGUMENT(SHLODOutliner*, Outliner)
		SLATE_ARGUMENT(UWorld*, World)
		SLATE_END_ARGS()

	public:
		/**
		* Construct child widgets that comprise this widget.
		*
		* @param InArgs Declaration from which to construct this widget.
		*/
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	public:
		// SMultiColumnTableRow overrides
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

		/** Returns World set through the HLODOutliner parent widget */
		UWorld* GetWorld() const { return World; }
	protected:
		/** Returns the display string of the node */
		FText GetItemDisplayString() const;

		/** Returns the tint of the node */
		FSlateColor GetTint() const;
		
		//~ Begin STableRow Interface.
		virtual void OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent) override;
		virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override;
		virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		//~ End STableRow Interface.

		

	private:
		/** The info about the widget that we are visualizing. */
		ITreeItem* TreeItem;

		/** Cached Display String as FText */
		FText CachedItemName;

		/** WeakPtr to the Treeview widget */
		TWeakPtr< STableViewBase > WeakTableViewBase;

		/** Pointer to the owning SHLODOutliner */
		SHLODOutliner* Outliner;
		
		/** Pointer to the currently represented world (unused atm) */
		UWorld* World;
	};
};
