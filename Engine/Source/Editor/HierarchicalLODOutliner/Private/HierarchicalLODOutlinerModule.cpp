// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchicalLODOutlinerModule.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Delegates/Delegate.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "HLOD/HLODSetup.h"
#include "HLODOutliner.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "IHierarchicalLODUtilities.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "UnrealEdGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

void FHierarchicalLODOutlinerModule::StartupModule()
{
	check(GUnrealEd);
	ArrayChangedDelegate = GUnrealEd->OnHLODLevelsArrayChanged().AddRaw(this, &FHierarchicalLODOutlinerModule::OnHLODLevelsArrayChangedEvent);
}

void FHierarchicalLODOutlinerModule::ShutdownModule()
{
	if (GUnrealEd)
	{
		GUnrealEd->OnHLODLevelsArrayChanged().Remove(ArrayChangedDelegate);
	}
}

void FHierarchicalLODOutlinerModule::OnHLODLevelsArrayChangedEvent()
{
	UWorld* CurrentWorld = nullptr;
	
	// Get the correct UWorld instance
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			CurrentWorld = Context.World();
			break;
		}
		else if (Context.WorldType == EWorldType::Editor)
		{
			CurrentWorld = Context.World();
		}
	}

	AWorldSettings* WorldSettings = (CurrentWorld) ? CurrentWorld->GetWorldSettings() : nullptr;

	if (WorldSettings)
	{
		TArray<FHierarchicalSimplification>& HierarchicalLODSetup = WorldSettings->GetHierarchicalLODSetup();
		int32 NumHLODLevels = WorldSettings->NumHLODLevels;

		if (HierarchicalLODSetup.Num() > 1 && HierarchicalLODSetup.Num() > NumHLODLevels)
		{
			// HLOD level was added, use previous level settings to "guess" new settings
			FHierarchicalSimplification& NewLevelSetup = HierarchicalLODSetup.Last();
			const FHierarchicalSimplification& OldLastLevelSetup = HierarchicalLODSetup[HierarchicalLODSetup.Num() - 2];

			NewLevelSetup.SimplificationMethod = OldLastLevelSetup.SimplificationMethod;
			NewLevelSetup.MergeSetting = OldLastLevelSetup.MergeSetting;
			NewLevelSetup.ProxySetting = OldLastLevelSetup.ProxySetting;
			NewLevelSetup.ApproximateSettings = OldLastLevelSetup.ApproximateSettings;

			NewLevelSetup.DesiredBoundRadius = OldLastLevelSetup.DesiredBoundRadius * 2.5f;
			NewLevelSetup.DesiredFillingPercentage = FMath::Max(OldLastLevelSetup.DesiredFillingPercentage * 0.75f, 1.0f);
			NewLevelSetup.TransitionScreenSize = OldLastLevelSetup.TransitionScreenSize * 0.75f;
			NewLevelSetup.MinNumberOfActorsToBuild = OldLastLevelSetup.MinNumberOfActorsToBuild;
		}
		else if (HierarchicalLODSetup.Num() < NumHLODLevels)
		{
			// HLOD Level was removed, now remove all LODActors for this level
			FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
			IHierarchicalLODUtilities* Utilities = Module.GetUtilities();
			Utilities->DeleteLODActorsInHLODLevel(CurrentWorld, NumHLODLevels - 1);
		}
	}
}

TSharedRef<SWidget> FHierarchicalLODOutlinerModule::CreateHLODOutlinerWidget()
{
	SAssignNew(HLODWindow, HLODOutliner::SHLODOutliner);		
	return HLODWindow->AsShared();
}

IMPLEMENT_MODULE(FHierarchicalLODOutlinerModule, HierarchicalLODOutliner);
