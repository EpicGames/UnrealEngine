// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "HierarchicalLODType.generated.h"

UENUM()	
enum class EHierarchicalLODActionType : uint8
{
	InvalidAction,
	CreateCluster,
	AddActorToCluster,
	MoveActorToCluster,
	RemoveActorFromCluster,
	MergeClusters,
	ChildCluster,

	MAX
};
