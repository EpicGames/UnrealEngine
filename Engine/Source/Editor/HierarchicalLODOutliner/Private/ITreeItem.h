// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HLODOutlinerDragDrop.h"
#include "Styling/SlateColor.h"
#include "Widgets/SWidget.h"

class UToolMenu;

namespace HLODOutliner
{
	class SHLODOutliner;
	struct FTreeItemID;

	/** Interface used for validating movement (ie, drag/dropping) operations */
	struct IDropTarget
	{
		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FDragValidationInfo ValidateDrop(FDragDropPayload& DraggedObjects) const = 0;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget) = 0;
	};
	
	typedef TSharedRef<ITreeItem> FTreeItemRef;

	struct ITreeItem : IDropTarget, TSharedFromThis < ITreeItem >
	{
		enum TreeItemType
		{
			Invalid,
			HierarchicalLODLevel,
			HierarchicalLODActor,
			StaticMeshActor
		};

		friend SHLODOutliner;

	protected:
		/** Default constructor */
		ITreeItem() : Parent(nullptr), Type(Invalid), bIsExpanded(false) {}
		virtual ~ITreeItem() = default;

		/** This item's parent, if any. */
		mutable TWeakPtr<ITreeItem> Parent;

		/** Array of children contained underneath this item */
		mutable TArray<TWeakPtr<ITreeItem>> Children;

		/** Item type enum */
		mutable TreeItemType Type;

	public:
		/** Get this item's parent. Can be nullptr. */
		FTreeItemPtr GetParent() const
		{
			return Parent.Pin();
		}

		/** Add a child to this item */
		void AddChild(FTreeItemRef Child)
		{
			Child->Parent = AsShared();
			Children.Add(MoveTemp(Child));
		}

		/** Remove a child from this item */
		void RemoveChild(const FTreeItemRef& Child)
		{
			if (Children.Remove(Child))
			{
				Child->Parent = nullptr;
			}
		}

		/** Returns the TreeItem's type */
		const TreeItemType GetTreeItemType()
		{
			return Type;
		}

		/** Get this item's children, if any. Although we store as weak pointers, they are guaranteed to be valid. */
		FORCEINLINE const TArray<TWeakPtr<ITreeItem>>& GetChildren() const
		{
			return Children;
		}

		/** Flag whether or not this item is expanded in the treeview */
		bool bIsExpanded;

	public:		
		/** Get the raw string to display for this tree item - used for sorting */
		virtual FString GetDisplayString() const = 0;

		/** Check whether it should be possible to interact with this tree item */
		virtual bool CanInteract() const = 0;

		/** Called when this item is expanded or collapsed */
		virtual void OnExpansionChanged() {};

		/** Generate a context menu for this item. Only called if *only* this item is selected. */
		virtual void GenerateContextMenu(UToolMenu* Menu, SHLODOutliner& Outliner) {};

		/** Populate the specified drag/drop payload with any relevant information for this type */
		virtual void PopulateDragDropPayload(FDragDropPayload& Payload) const = 0;
		
		/** Returns this TreeItem's ID */
		virtual FTreeItemID GetID() = 0;

		/** Returns the Tint used for displaying this item in the Treeview*/
		virtual FSlateColor GetTint() const
		{
			return FLinearColor(1.0f, 1.0f, 1.0f);
		}
	};
};
