// Copyright Epic Games, Inc. All Rights Reserved.

#include "LODActorItem.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Engine/Level.h"
#include "Framework/Commands/UIAction.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HLODOutliner.h"
#include "HierarchicalLODType.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "IHierarchicalLODUtilities.h"
#include "ITreeItem.h"
#include "Internationalization/Internationalization.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

class SWidget;

#define LOCTEXT_NAMESPACE "LODActorItem"

HLODOutliner::FLODActorItem::FLODActorItem(ALODActor* InLODActor)
	: LODActor(InLODActor)
	, ID(InLODActor)
{
	Type = ITreeItem::HierarchicalLODActor;
}


bool HLODOutliner::FLODActorItem::CanInteract() const
{
	return true;
}

void HLODOutliner::FLODActorItem::GenerateContextMenu(UToolMenu* Menu, SHLODOutliner& Outliner)
{
	auto SharedOutliner = StaticCastSharedRef<SHLODOutliner>(Outliner.AsShared());

	FToolMenuSection& Section = Menu->AddSection("Section");
	Section.AddMenuEntry("SelectLODActor", LOCTEXT("SelectLODActor", "Select LOD Actor"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::SelectLODActor)));

	Section.AddMenuEntry("SelectContainedActors", LOCTEXT("SelectContainedActors", "Select Contained Actors"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::SelectContainedActors)));

	if (!LODActor->IsBuilt())
	{
		Section.AddMenuEntry("BuildLODActorMesh", LOCTEXT("BuildLODActorMesh", "Build Proxy Mesh"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::BuildLODActor)));
	}
	else
	{
		Section.AddMenuEntry("ForceView", LOCTEXT("ForceView", "ForceView"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::ForceViewLODActor)));
		Section.AddMenuEntry("RebuildLODActorMesh", LOCTEXT("RebuildLODActorMesh", "Rebuild Proxy Mesh"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::RebuildLODActor)));
	}

	Section.AddMenuEntry("CreateHLODVolume", LOCTEXT("CreateHLODVolume", "Create Containing Hierarchical Volume"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::CreateHierarchicalVolumeForActor)));

	AActor* Actor = LODActor.Get();
	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();
	ALODActor* ParentActor = Utilities->GetParentLODActor(Actor);
	if (ParentActor && Parent.Pin()->GetTreeItemType() == TreeItemType::HierarchicalLODActor)
	{		
		Section.AddMenuEntry("RemoveChildFromCluster", LOCTEXT("RemoveChildFromCluster", "Remove from cluster"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::RemoveLODActorFromCluster)));
	}
	else
	{
		Section.AddMenuEntry("DeleteCluster", LOCTEXT("DeleteCluster", "Delete Cluster"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::DeleteCluster)));
	}
}

FString HLODOutliner::FLODActorItem::GetDisplayString() const
{
	if (ALODActor* ActorPtr = LODActor.Get())
	{
		FString DisplayName = ActorPtr->GetName() + (!ActorPtr->HasValidSubActors() ? FString(" (Not enough mesh components)") : ((!ActorPtr->IsBuilt()) ? FString(" (Not built)") : FString()));

		// Temporary indication of custom settings
		if (ActorPtr->bOverrideMaterialMergeSettings || ActorPtr->bOverrideScreenSize || ActorPtr->bOverrideTransitionScreenSize)
		{
			DisplayName += " (Using Cluster-based settings)";
		}

		return DisplayName;
	}

	return FString();
}

HLODOutliner::FTreeItemID HLODOutliner::FLODActorItem::GetID()
{
	return ID;
}

FSlateColor HLODOutliner::FLODActorItem::GetTint() const
{
	ALODActor* LODActorPtr = LODActor.Get();
	if (LODActorPtr)
	{
		return !LODActorPtr->IsBuilt() ? FSlateColor::UseSubduedForeground() : FLinearColor(1.0f, 1.0f, 1.0f);
	}

	return FLinearColor(1.0f, 1.0f, 1.0f);
}

FText HLODOutliner::FLODActorItem::GetRawNumTrianglesAsText() const
{
	if (LODActor.IsValid())
	{
		const uint32 SubActorsTriangleCount = LODActor->GetNumTrianglesInSubActors();
		return FText::FromString(FString::FromInt(SubActorsTriangleCount));
	}
	else
	{
		return FText::FromString(TEXT("Not available"));
	}
}

FText HLODOutliner::FLODActorItem::GetReducedNumTrianglesAsText() const
{
	if (LODActor.IsValid())
	{
		const uint32 MergedCount = LODActor->GetNumTrianglesInMergedMesh();
		return FText::FromString(FString::FromInt(MergedCount));
	}
	else
	{
		return FText::FromString(TEXT("Not available"));
	}
}

FText HLODOutliner::FLODActorItem::GetReductionPercentageAsText() const
{
	if (LODActor.IsValid())
	{
		const uint32 SubActorCount = LODActor->GetNumTrianglesInSubActors();
		const uint32 MergedCount = LODActor->GetNumTrianglesInMergedMesh();
		const float PercentageOfOriginal = ((float)MergedCount / (float)SubActorCount) * 100.0f;
		return  FText::FromString("%" + ((SubActorCount != 0) ? FString::SanitizeFloat(PercentageOfOriginal) : TEXT("0")));
	}
	else
	{
		return FText::FromString(TEXT("Not available"));
	}
	
}

FText HLODOutliner::FLODActorItem::GetLevelAsText() const
{
	if (LODActor.IsValid())
	{
		return FText::FromString(LODActor->SubActors.Num() ? LODActor->SubActors[0]->GetLevel()->GetOuter()->GetName() : TEXT(""));
	}
	else
	{
		return FText::FromString(TEXT("Not available"));
	}
}

void HLODOutliner::FLODActorItem::PopulateDragDropPayload(FDragDropPayload& Payload) const
{
	ALODActor* ActorPtr = LODActor.Get();
	if (ActorPtr)
	{
		if (!Payload.LODActors)
		{
			Payload.LODActors = TArray<TWeakObjectPtr<AActor>>();
		}
		Payload.LODActors->Add(LODActor);
	}
}

HLODOutliner::FDragValidationInfo HLODOutliner::FLODActorItem::ValidateDrop(FDragDropPayload& DraggedObjects) const
{	
	if (Parent.IsValid())
	{
		if (Parent.Pin()->GetTreeItemType() == ITreeItem::HierarchicalLODActor)
		{
			return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("CannotAddToChildCluster", "Cannot Add to Child cluster"));
		}
	}

	FLODActorDropTarget Target(LODActor.Get());
	return Target.ValidateDrop(DraggedObjects);
}

void HLODOutliner::FLODActorItem::OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget)
{
	FLODActorDropTarget Target(LODActor.Get());
	Target.OnDrop(DraggedObjects, ValidationInfo, DroppedOnWidget);

	// Expand this HLOD actor item
	bIsExpanded = true;	
}

HLODOutliner::FDragValidationInfo HLODOutliner::FLODActorDropTarget::ValidateDrop(FDragDropPayload& DraggedObjects) const
{
	if (DraggedObjects.StaticMeshActors.IsSet() && DraggedObjects.StaticMeshActors->Num() > 0)
	{
		if (DraggedObjects.LODActors->Num() == 0)
		{			
			if (DraggedObjects.bSceneOutliner == false)
			{
				bool bContaining = false;

				// Check if this StaticMesh Actor is not already inside this cluster
				for (int32 ActorIndex = 0; ActorIndex < DraggedObjects.StaticMeshActors->Num(); ++ActorIndex)
				{
					if (LODActor->SubActors.Contains(DraggedObjects.StaticMeshActors.GetValue()[ActorIndex]))
					{
						bContaining = true;
						break;
					}
				}

				if (!bContaining)
				{
					if (DraggedObjects.StaticMeshActors->Num() > 1)
					{
						return FDragValidationInfo(EHierarchicalLODActionType::MoveActorToCluster, FHLODOutlinerDragDropOp::ToolTip_MultipleSelection_Compatible, LOCTEXT("MoveMultipleToCluster", "Move Actors to Cluster"));
					}
					else
					{
						return FDragValidationInfo(EHierarchicalLODActionType::MoveActorToCluster, FHLODOutlinerDragDropOp::ToolTip_Compatible, LOCTEXT("MoveToCluster", "Move Actor to Cluster"));
					}
				}
				else
				{
					return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("AlreadyInCluster", "Cannot Add to Existing cluster"));
				}
			}
			else
			{
				TArray<AActor*> Actors;
				for (auto StaticMeshActor : DraggedObjects.StaticMeshActors.GetValue())
				{
					Actors.Add(StaticMeshActor.Get());
				}

				Actors.Add(LODActor.Get());
				
				const bool MultipleActors = DraggedObjects.StaticMeshActors->Num() > 1;
				FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
				IHierarchicalLODUtilities* Utilities = Module.GetUtilities();
				if (Utilities->AreActorsInSamePersistingLevel(Actors))
				{
					return FDragValidationInfo(EHierarchicalLODActionType::AddActorToCluster, 
						MultipleActors ? FHLODOutlinerDragDropOp::ToolTip_MultipleSelection_Compatible : FHLODOutlinerDragDropOp::ToolTip_Compatible,
						MultipleActors ? LOCTEXT("AddMultipleToCluster", "Add Actors to Cluster") : LOCTEXT("AddToCluster", "Add Actor to Cluster"));
				}
				else
				{
					return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction,
						MultipleActors ? FHLODOutlinerDragDropOp::ToolTip_MultipleSelection_Incompatible : FHLODOutlinerDragDropOp::ToolTip_Incompatible,
						LOCTEXT("NotInSameLODLevel", "Actors are not all in the same persisting level"));
				}				
			}			
		}

		if (DraggedObjects.bSceneOutliner)
		{
			return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("AlreadyInHLOD", "Actor is already in one of the Hierarchical LOD clusters"));
		}
	}
	else if (DraggedObjects.LODActors.IsSet() && DraggedObjects.LODActors->Num() > 0)
	{
		// Only valid if dragging inside the HLOD outliner
		if (!DraggedObjects.bSceneOutliner)
		{		
			bool bValidForMerge = true;
			bool bValidForChilding = true;
			int32 FirstLODLevel = -1;
			UObject* LevelOuter = nullptr;
			UObject* SubActorOuter = (LODActor->SubActors.Num()) ? LODActor->SubActors[0]->GetOuter() : nullptr;

			for (auto Actor : DraggedObjects.LODActors.GetValue())
			{
				ALODActor* InLODActor = Cast<ALODActor>(Actor.Get());			

				if (InLODActor)
				{
					// If dragged onto self or already containing LODActor early out
					if (InLODActor == LODActor.Get() || LODActor->SubActors.Contains(InLODActor))
					{
						bValidForMerge = false;
						bValidForChilding = false;
						break;
					}

					// Check in case of multiple selected LODActor items to make sure all of them come from the same LOD level
					if (FirstLODLevel == -1)
					{
						FirstLODLevel = InLODActor->LODLevel;
					}

					if (InLODActor->LODLevel != LODActor->LODLevel)
					{
						bValidForMerge = false;

						if (InLODActor->LODLevel != FirstLODLevel)
						{
							bValidForChilding = false;
						}
					}		

					// Check if in same level asset
					if (LevelOuter == nullptr)
					{
						LevelOuter = InLODActor->GetOuter();
					}
					else if (LevelOuter != InLODActor->GetOuter())
					{
						bValidForMerge = false;
						bValidForChilding = false;
					}
					
					if (InLODActor->SubActors.Num() && SubActorOuter != InLODActor->SubActors[0]->GetOuter())
					{
						bValidForChilding = false;
						bValidForMerge = false;
					}					
					
				}
			}

			if (bValidForMerge)
			{
				return FDragValidationInfo(EHierarchicalLODActionType::MergeClusters, (DraggedObjects.LODActors->Num() == 1) ? FHLODOutlinerDragDropOp::ToolTip_Compatible : FHLODOutlinerDragDropOp::ToolTip_MultipleSelection_Compatible, LOCTEXT("MergeHLODClusters", "Merge Cluster(s)"));
			}
			else if (bValidForChilding)
			{
				return FDragValidationInfo(EHierarchicalLODActionType::ChildCluster, (DraggedObjects.LODActors->Num() == 1) ? FHLODOutlinerDragDropOp::ToolTip_Compatible : FHLODOutlinerDragDropOp::ToolTip_MultipleSelection_Compatible, LOCTEXT("ChildHLODClusters", "Add Child Cluster(s)"));
			}
			else
			{
				return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("InvalidOperation", "Invalid Operation"));
			}
		}
	}

	return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("NotImplemented", "Not implemented"));
}

void HLODOutliner::FLODActorDropTarget::OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget)
{
	AActor* DropActor = LODActor.Get();
	if (!DropActor)
	{
		return;
	}

	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

	auto& DraggedStaticMeshActors = DraggedObjects.StaticMeshActors.GetValue();
	auto& DraggedLODActors = DraggedObjects.LODActors.GetValue();
	if (ValidationInfo.ActionType == EHierarchicalLODActionType::AddActorToCluster || ValidationInfo.ActionType == EHierarchicalLODActionType::MoveActorToCluster)
	{
		for (int32 ActorIndex = 0; ActorIndex < DraggedStaticMeshActors.Num(); ++ActorIndex)
		{
			auto Actor = DraggedStaticMeshActors[ActorIndex];
			Utilities->AddActorToCluster(Actor.Get(), LODActor.Get());
		}
	}
	else if (ValidationInfo.ActionType == EHierarchicalLODActionType::MergeClusters)
	{
		for (int32 ActorIndex = 0; ActorIndex < DraggedLODActors.Num(); ++ActorIndex)
		{
			ALODActor* InLODActor = Cast<ALODActor>(DraggedLODActors[ActorIndex].Get());			
			Utilities->MergeClusters(LODActor.Get(), InLODActor);
		}
	}
	else if (ValidationInfo.ActionType == EHierarchicalLODActionType::ChildCluster)
	{
		for (int32 ActorIndex = 0; ActorIndex < DraggedLODActors.Num(); ++ActorIndex)
		{
			auto Actor = DraggedLODActors[ActorIndex];			
			Utilities->AddActorToCluster(Actor.Get(), LODActor.Get());
		}
	}
}

#undef LOCTEXT_NAMESPACE
