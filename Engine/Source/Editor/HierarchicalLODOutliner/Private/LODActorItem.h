// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "Engine/LODActor.h"
#include "HLODOutlinerDragDrop.h"
#include "Internationalization/Text.h"
#include "ITreeItem.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "TreeItemID.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SWidget;
class UToolMenu;

namespace HLODOutliner
{
class SHLODOutliner;

	/** Helper class to manage moving arbitrary data onto an actor */
	struct FLODActorDropTarget : IDropTarget
	{
		/** The actor this tree item is associated with. */
		TWeakObjectPtr<ALODActor> LODActor;

		/** Construct this object out of an Actor */
		FLODActorDropTarget(ALODActor* InLODActor) : LODActor(InLODActor) {}

	public:
		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FDragValidationInfo ValidateDrop(FDragDropPayload& DraggedObjects) const override;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget) override;
	};

	struct FLODActorItem : ITreeItem
	{
		mutable TWeakObjectPtr<ALODActor> LODActor;
		mutable FTreeItemID ID;

		explicit FLODActorItem(ALODActor* InLODActor);

		//~ Begin ITreeItem Interface.
		virtual bool CanInteract() const override;
		virtual void GenerateContextMenu(UToolMenu* Menu, SHLODOutliner& Outliner) override;
		virtual FString GetDisplayString() const override;
		virtual FSlateColor GetTint() const override;
		virtual FTreeItemID GetID() override;
		//~ End ITreeItem Interface.

		/** Returns the number of triangles for the represented LODActor in FText form */
		FText GetRawNumTrianglesAsText() const;

		/** Returns the reduced number of triangles for the represented LODActor in FText form */
		FText GetReducedNumTrianglesAsText() const;

		/** Returns the reduction percentage for the represented LODActor in FText form */
		FText GetReductionPercentageAsText() const;

		/** Returns the level the meshes are in Ftext form */
		FText GetLevelAsText() const;

		

		/** Populate the specified drag/drop payload with any relevant information for this type */
		virtual void PopulateDragDropPayload(FDragDropPayload& Payload) const override;

		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FDragValidationInfo ValidateDrop(FDragDropPayload& DraggedObjects) const override;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget) override;
	};
};
