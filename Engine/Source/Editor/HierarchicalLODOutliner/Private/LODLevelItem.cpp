// Copyright Epic Games, Inc. All Rights Reserved.

#include "LODLevelItem.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/LODActor.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Framework/Commands/UIAction.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HLODOutliner.h"
#include "HierarchicalLODType.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "IHierarchicalLODUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "Selection.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SWidget;

#define LOCTEXT_NAMESPACE "LODLevelItem"

HLODOutliner::FLODLevelItem::FLODLevelItem(const uint32 InLODIndex)
	: LODLevelIndex(InLODIndex), ID(nullptr)
{
	Type = ITreeItem::HierarchicalLODLevel;
	ID.SetCachedHash(GetTypeHash(FString("LODLevel - ") + FString::FromInt(LODLevelIndex)));
}

bool HLODOutliner::FLODLevelItem::CanInteract() const
{
	return true;
}

void HLODOutliner::FLODLevelItem::GenerateContextMenu(UToolMenu* Menu, SHLODOutliner& Outliner)
{
	// Collect actors / components available for creating a new HLOD cluster 
	USelection* CurrentSelection = GEditor->GetSelectedActors();	
	if (const int32 NumSelected = CurrentSelection->Num())
	{
		TArray<AActor*> ValidActorsToCluster;
		for (int32 SelectionIndex = 0; SelectionIndex < NumSelected; ++SelectionIndex)
		{
			AActor* Actor = Cast<AActor>(CurrentSelection->GetSelectedObject(SelectionIndex));
			if (Actor)
			{
				if (Outliner.HierarchicalLODUtilities->ShouldGenerateCluster(Actor, LODLevelIndex) == EClusterGenerationError::ValidActor)
				{
					ValidActorsToCluster.Add(Actor);
				}
			}
		}

		// Generate context menu with option to create a new cluster from the current viewport selection
		FToolMenuSection& Section = Menu->AddSection("Section");
		Section.AddMenuEntry("CreateClusterFromViewportSelection", LOCTEXT("CreateClusterFromViewportSelection", "Create Cluster from World Selection"), FText(), FSlateIcon(), 
			FUIAction(FExecuteAction::CreateLambda([&Outliner, ValidActorsToCluster, this]()
			{
				Outliner.CreateClusterFromActors(ValidActorsToCluster, LODLevelIndex);
			}),
				FCanExecuteAction::CreateLambda([ValidActorsToCluster]() -> bool
			{ 
				return (ValidActorsToCluster.Num() > 1); 
			})));		
	}
}

FString HLODOutliner::FLODLevelItem::GetDisplayString() const
{
	return FString("LODLevel - ") + FString::FromInt(LODLevelIndex);
}

HLODOutliner::FTreeItemID HLODOutliner::FLODLevelItem::GetID()
{
	return ID;
}

void HLODOutliner::FLODLevelItem::PopulateDragDropPayload(FDragDropPayload& Payload) const
{
	// Nothing to populate
}

HLODOutliner::FDragValidationInfo HLODOutliner::FLODLevelItem::ValidateDrop(FDragDropPayload& DraggedObjects) const
{
	FLODLevelDropTarget Target(LODLevelIndex);
	return Target.ValidateDrop(DraggedObjects);
}

void HLODOutliner::FLODLevelItem::OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget)
{
	FLODLevelDropTarget Target(LODLevelIndex);		
	Target.OnDrop(DraggedObjects, ValidationInfo, DroppedOnWidget);

	// Expand this HLOD level item
	bIsExpanded = true;
}

HLODOutliner::FDragValidationInfo HLODOutliner::FLODLevelDropTarget::ValidateDrop(FDragDropPayload& DraggedObjects) const
{
	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

	if (DraggedObjects.StaticMeshActors.IsSet() && DraggedObjects.StaticMeshActors->Num() > 0)
	{
		const int32 NumStaticMeshActors = DraggedObjects.StaticMeshActors->Num();	
		
		ULevel* Level = nullptr;

		TArray<AActor*> DraggedActors;

		for (auto Actor : DraggedObjects.StaticMeshActors.GetValue())
		{
			DraggedActors.Add(Actor.Get());
		}

		bool bSameLevelInstance = Utilities->AreActorsInSamePersistingLevel(DraggedActors);
		bool bAlreadyClustered = Utilities->AreActorsClustered(DraggedActors);
		
		if (!bSameLevelInstance)
		{
			return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("StaticMeshActorsNotInSameLevelAsset", "Static Mesh Actors not in the same level asset (streaming level)"));
		}

		if (bAlreadyClustered && DraggedObjects.bSceneOutliner )
		{
			return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("AlreadyClusters", "One or more Static Mesh Actors is already in a cluster"));
		}

		return FDragValidationInfo(EHierarchicalLODActionType::CreateCluster, FHLODOutlinerDragDropOp::ToolTip_Compatible, LOCTEXT("CreateNewCluster", "Create new Cluster"));
	}
	else if (DraggedObjects.LODActors.IsSet() && DraggedObjects.LODActors->Num() > 0)
	{
		const int32 NumLODActors = DraggedObjects.LODActors->Num();

		if (NumLODActors > 1)
		{
			// Gather LOD actors for checks
			auto LODActors = DraggedObjects.LODActors.GetValue();			
			TArray<ALODActor*> DraggedLODActors;
			TArray<AActor*> DraggedActors;
			for (auto Actor : LODActors)
			{
				DraggedLODActors.Add(Cast<ALODActor>(Actor.Get()));
				DraggedActors.Add(Actor.Get());
			}

			bool bSameLevelInstance = Utilities->AreActorsInSamePersistingLevel(DraggedActors);
			bool bSameLODLevel = Utilities->AreClustersInSameHLODLevel(DraggedLODActors);
			const uint32 LevelIndex = (DraggedLODActors.Num() > 0 ) ? DraggedLODActors[0]->LODLevel : 0;
			bool bIsClustered = Utilities->AreActorsClustered(DraggedActors);

			if (!bSameLODLevel)
			{
				return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("NotInSameLODLevel", "LODActors are not all in the same HLOD level"));
			}

			if (!bSameLevelInstance)
			{
				return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("LODActorsNotInSameLevelAsset", "LODActors not in the same level asset (streaming level)"));
			}
						
			if (bSameLevelInstance && bSameLODLevel && LevelIndex < (int32)(LODLevelIndex + 1))
			{
				return FDragValidationInfo(EHierarchicalLODActionType::CreateCluster, FHLODOutlinerDragDropOp::ToolTip_MultipleSelection_Compatible, LOCTEXT("CreateNewCluster", "Create new Cluster"));
			}			
		}
	}

	return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("NotImplemented", "Not implemented"));
}				

void HLODOutliner::FLODLevelDropTarget::OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget)
{
	if (ValidationInfo.ActionType == EHierarchicalLODActionType::CreateCluster)
	{
		CreateNewCluster(DraggedObjects);
	}	
}

void HLODOutliner::FLODLevelDropTarget::CreateNewCluster(FDragDropPayload &DraggedObjects)
{	
	// Outerworld in which the LODActors should be spawned/saved (this is to enable support for streaming levels)
	UWorld* OuterWorld = nullptr;
	if (DraggedObjects.StaticMeshActors.IsSet() && DraggedObjects.StaticMeshActors->Num() > 0)
	{
		OuterWorld = Cast<UWorld>(DraggedObjects.StaticMeshActors.GetValue()[0]->GetLevel()->GetOuter());
	}
	else if (DraggedObjects.LODActors.IsSet() && DraggedObjects.LODActors->Num() > 0)
	{
		OuterWorld = Cast<UWorld>(DraggedObjects.LODActors.GetValue()[0]->GetLevel()->GetOuter());
	}

	// Gather sub actors from the drag and drop operation
	TArray<AActor*> SubActors;
	for (TWeakObjectPtr<AActor> StaticMeshActor : DraggedObjects.StaticMeshActors.GetValue())
	{
		SubActors.Add(StaticMeshActor.Get());
	}

	for (TWeakObjectPtr<AActor> LODActor : DraggedObjects.LODActors.GetValue())
	{
		SubActors.Add(LODActor.Get());
	}

	FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = Module.GetUtilities();
	// Create the new cluster
	Utilities->CreateNewClusterFromActors(OuterWorld, DraggedObjects.OutlinerWorld->GetWorldSettings(), SubActors, LODLevelIndex);	
}

#undef LOCTEXT_NAMESPACE
