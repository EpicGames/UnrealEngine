// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HLODOutlinerDragDrop.h"
#include "ITreeItem.h"
#include "Templates/SharedPointer.h"
#include "TreeItemID.h"

class SWidget;
class UToolMenu;

namespace HLODOutliner
{
class SHLODOutliner;

	/** Helper class to manage moving abritrary data onto an actor */
	struct FLODLevelDropTarget : IDropTarget
	{
		/** The actor this tree item is associated with. */
		uint32 LODLevelIndex;

		/** Construct this object out of an Actor */
		FLODLevelDropTarget(uint32 InLODIndex) : LODLevelIndex(InLODIndex) {}

	public:

		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FDragValidationInfo ValidateDrop(FDragDropPayload& DraggedObjects) const override;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget) override;

		/** Creating a new cluster (LODActor) with given objects/actors */
		void CreateNewCluster(FDragDropPayload &DraggedObjects);
	};

	struct FLODLevelItem : ITreeItem
	{
		FLODLevelItem(const uint32 InLODIndex);

		//~ Begin ITreeItem Interface.
		virtual bool CanInteract() const override;
		virtual void GenerateContextMenu(UToolMenu* Menu, SHLODOutliner& Outliner) override;
		virtual FString GetDisplayString() const override;
		virtual FTreeItemID GetID() override;
		//~ Begin ITreeItem Interface.

		/** Populate the specified drag/drop payload with any relevant information for this type */
		virtual void PopulateDragDropPayload(FDragDropPayload& Payload) const override;

		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FDragValidationInfo ValidateDrop(FDragDropPayload& DraggedObjects) const override;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget) override;
		
		/** LOD index of this level */
		uint32 LODLevelIndex;

		/** TreeItem ID */
		FTreeItemID ID;
	};
};
