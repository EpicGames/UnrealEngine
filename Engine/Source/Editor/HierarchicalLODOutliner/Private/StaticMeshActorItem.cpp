// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshActorItem.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HLODOutliner.h"
#include "HierarchicalLODType.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

class SWidget;

#define LOCTEXT_NAMESPACE "StaticMeshActorItem"

HLODOutliner::FStaticMeshActorItem::FStaticMeshActorItem(AActor* InStaticMeshActor)
	: StaticMeshActor(InStaticMeshActor), ID(InStaticMeshActor)
{
	Type = ITreeItem::StaticMeshActor;
}

bool HLODOutliner::FStaticMeshActorItem::CanInteract() const
{
	return true;
}

void HLODOutliner::FStaticMeshActorItem::GenerateContextMenu(UToolMenu* Menu, SHLODOutliner& Outliner)
{
	auto SharedOutliner = StaticCastSharedRef<SHLODOutliner>(Outliner.AsShared());
	FToolMenuSection& Section = Menu->AddSection("Section");
	Section.AddMenuEntry("RemoveSMActorFromCluster", LOCTEXT("RemoveSMActorFromCluster", "Remove From Cluster"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::RemoveStaticMeshActorFromCluster)));
	Section.AddMenuEntry("ExcludeSMActorFromGeneration", LOCTEXT("ExcludeSMActorFromGeneration", "Exclude From Cluster Generation"), FText(), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(&Outliner, &SHLODOutliner::ExcludeFromClusterGeneration)));
}

FString HLODOutliner::FStaticMeshActorItem::GetDisplayString() const
{
	if (StaticMeshActor.IsValid())
	{
		return StaticMeshActor->GetActorNameOrLabel();
	}
	else
	{
		return FString("");
	}	
}

HLODOutliner::FTreeItemID HLODOutliner::FStaticMeshActorItem::GetID()
{
	return ID;
}

void HLODOutliner::FStaticMeshActorItem::PopulateDragDropPayload(FDragDropPayload& Payload) const
{
	AActor* ActorPtr = StaticMeshActor.Get();
	if (ActorPtr)
	{
		if (!Payload.StaticMeshActors)
		{
			Payload.StaticMeshActors = TArray<TWeakObjectPtr<AActor>>();
		}
		Payload.StaticMeshActors->Add(StaticMeshActor);		
	}
}

HLODOutliner::FDragValidationInfo HLODOutliner::FStaticMeshActorItem::ValidateDrop(FDragDropPayload& DraggedObjects) const
{
	FStaticMeshActorDropTarget Target(StaticMeshActor.Get());
	return Target.ValidateDrop(DraggedObjects);
}

void HLODOutliner::FStaticMeshActorItem::OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget)
{
	FStaticMeshActorDropTarget Target(StaticMeshActor.Get());
	Target.OnDrop(DraggedObjects, ValidationInfo, DroppedOnWidget);
}


HLODOutliner::FDragValidationInfo HLODOutliner::FStaticMeshActorDropTarget::ValidateDrop(FDragDropPayload& DraggedObjects) const
{
	return FDragValidationInfo(EHierarchicalLODActionType::InvalidAction, FHLODOutlinerDragDropOp::ToolTip_Incompatible, LOCTEXT("NotImplemented", "Not implemented"));
}

void HLODOutliner::FStaticMeshActorDropTarget::OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget)
{

}

#undef LOCTEXT_NAMESPACE
