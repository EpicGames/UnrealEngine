// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "GameFramework/Actor.h"
#include "HLODOutlinerDragDrop.h"
#include "ITreeItem.h"
#include "Templates/SharedPointer.h"
#include "TreeItemID.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SWidget;
class UToolMenu;

namespace HLODOutliner
{
	class SHLODOutliner;

	/** Helper class to manage moving abritrary data onto an actor */
	struct FStaticMeshActorDropTarget : IDropTarget
	{
		/** The actor this tree item is associated with. */
		TWeakObjectPtr<AActor> Actor;

		/** Construct this object out of an Actor */
		FStaticMeshActorDropTarget(AActor* InActor) : Actor(InActor) {}
	public:
		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FDragValidationInfo ValidateDrop(FDragDropPayload& DraggedObjects) const override;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget) override;
	};

	struct FStaticMeshActorItem : ITreeItem
	{
		/** Represented StaticMeshActor */
		mutable TWeakObjectPtr<AActor> StaticMeshActor;
		/** TreeItem's ID */
		mutable FTreeItemID ID;

		explicit FStaticMeshActorItem(AActor* InStaticMeshActor);

		//~ Begin ITreeItem Interface.
		virtual bool CanInteract() const override;
		virtual void GenerateContextMenu(UToolMenu* Menu, SHLODOutliner& Outliner) override;
		virtual FString GetDisplayString() const override;
		virtual FTreeItemID GetID() override;
		//~ End ITreeItem Interface.

		/** Populate the specified drag/drop payload with any relevant information for this type */
		virtual void PopulateDragDropPayload(FDragDropPayload& Payload) const override;

		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FDragValidationInfo ValidateDrop(FDragDropPayload& DraggedObjects) const override;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(FDragDropPayload& DraggedObjects, const FDragValidationInfo& ValidationInfo, TSharedRef<SWidget> DroppedOnWidget) override;
	};
};
