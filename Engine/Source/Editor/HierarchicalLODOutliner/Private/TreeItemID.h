// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectKey.h"

namespace HLODOutliner
{
	struct FTreeItemID
	{
	public:
		FTreeItemID() : CachedHash(0) {}

		FTreeItemID(const UObject* InObject)
		{
			ObjectKey = FObjectKey(InObject);
			CachedHash = GetTypeHash(ObjectKey);
		}

		/** Copy construction / assignment */
		FTreeItemID(const FTreeItemID& Other)
		{
			*this = Other;
		}

		FTreeItemID& operator=(const FTreeItemID& Other)
		{
			ObjectKey = Other.ObjectKey;
			CachedHash = Other.CachedHash;
			return *this;
		}

		/** Move construction / assignment */
		FTreeItemID(FTreeItemID&& Other)
		{
			*this = MoveTemp(Other);
		}

		FTreeItemID& operator=(FTreeItemID&& Other)
		{
			Swap(*this, Other);
			return *this;
		}

		~FTreeItemID()
		{
			CachedHash = 0;
		}

		friend bool operator==(const FTreeItemID& One, const FTreeItemID& Other)
		{
			return One.CachedHash == Other.CachedHash;
		}
		friend bool operator!=(const FTreeItemID& One, const FTreeItemID& Other)
		{
			return One.CachedHash != Other.CachedHash;
		}

		friend uint32 GetTypeHash(const FTreeItemID& ItemID)
		{
			return ItemID.CachedHash;
		}

		void SetCachedHash(const uint32 InHash)
		{
			CachedHash = InHash;
		}
	private:
		FObjectKey ObjectKey;
		uint32 CachedHash;
	};
};
