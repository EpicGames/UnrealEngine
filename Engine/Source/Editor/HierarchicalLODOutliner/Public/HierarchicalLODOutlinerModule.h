// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class SWidget;
namespace HLODOutliner { class SHLODOutliner; };

/**
* The module holding all of the UI related pieces for HLOD Outliner
*/
class FHierarchicalLODOutlinerModule : public IModuleInterface
{
public:
	/**
	* Called right after the module DLL has been loaded and the module object has been created
	*/
	virtual void StartupModule();

	/**
	* Called before the module is unloaded, right before the module object is destroyed.
	*/
	virtual void ShutdownModule();

	void OnHLODLevelsArrayChangedEvent();

	/** Creates the HLOD Outliner widget */
	virtual TSharedRef<SWidget> CreateHLODOutlinerWidget();

private:
	FDelegateHandle ArrayChangedDelegate;
	TSharedPtr<HLODOutliner::SHLODOutliner> HLODWindow;
};
